// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Base class of any editor-only test objects
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Script.h"
#include "UObject/ScriptMacros.h"
#include "Containers/Ticker.h"

#include "EditorUtilityTest.generated.h"

UENUM(BlueprintType)
enum class EEditorUtilityTestResult : uint8
{
	/**
	 * When finishing a test if you use Default, you're not explicitly stating if the test passed or failed.
	 * Instead you're allowing any reported failure to have decided that for you. Even if you do
	 * explicitly set as a success, it can be overturned by errors that occurred during the test.
	 */
	Default,
	Invalid,
	Preparing,
	Running,
	Failed,
	Succeeded
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FEditorUtilityTestEventSignature);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FEditorUtilityTestFinishedSignature, EEditorUtilityTestResult, TestState);

UCLASS(Blueprintable, meta = (ShowWorldContextPin))
class UEditorUtilityTest : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/** Called when the test is ready to prepare */
	UPROPERTY(BlueprintAssignable)
	FEditorUtilityTestEventSignature OnTestPrepare;

	/** Called when the test is started */
	UPROPERTY(BlueprintAssignable)
	FEditorUtilityTestEventSignature OnTestStart;

	/** Called when the test is finished. Use it to clean up */
	UPROPERTY(BlueprintAssignable)
	FEditorUtilityTestFinishedSignature OnTestFinished;

	/**
	 * The owner is the group or person responsible for the test. Generally you should use a group name
	 * like 'Editor' or 'Rendering'. When a test fails it may not be obvious who should investigate
	 * so this provides a associate responsible groups with tests.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Context", meta = (MultiLine = "true"))
	FString Owner;

	/** A description of the test, like what is this test trying to determine. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Context", meta = (MultiLine = "true"))
	FString Description;

	/** The Test's time limit for preparation, this is the time it has to trigger IsReadyToStart(). '0' means no limit. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Timeout")
	float PreparationTimeLimit;

	/** Test's total run time limit. '0' means no limit */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Timeout")
	float TimeLimit;

protected:
	/** Blueprint Utility Editor entry point */
	UFUNCTION(BlueprintCallable, Category = "Editor Utility Test")
	void Run();

	/** Use to setup test before running, must call FinishPrepareTest at the end to actual start the test. */
	UFUNCTION(BlueprintNativeEvent, Category = "Editor Utility Test")
	void PrepareTest();

	/** Tell the blueprint VM to start the test (to use at the end of Prepare Test event). */
	UFUNCTION(BlueprintCallable, Category = "Editor Utility Test")
	virtual void FinishPrepareTest();

	/** Actual run the test, must call FinishTest at then end of test with a state to signify the test is done. */
	UFUNCTION(BlueprintNativeEvent, Category = "Editor Utility Test")
	void StartTest();

	/** Tell the VM the test is finished with specify state. */
	UFUNCTION(BlueprintCallable, Category = "Editor Utility Test", meta = (AdvancedDisplay = "Message"))
	virtual void FinishTest(EEditorUtilityTestResult TestState, const FString& Message);

	/** Use to add clean up steps, the call is blocking. */
	UFUNCTION(BlueprintImplementableEvent, Category = "Editor Utility Test", meta = (DisplayName = "Finished Test"))
	void ReceiveFinishedTest(EEditorUtilityTestResult TestState, EEditorUtilityTestResult& FinalState);

	/** Add Error */
	UFUNCTION(BlueprintCallable, Category = "Editor Utility Test")
	virtual void AddError(const FString& Message);

	/** Add Warning */
	UFUNCTION(BlueprintCallable, Category = "Editor Utility Test")
	virtual void AddWarning(const FString& Message);

	/** Add Info */
	UFUNCTION(BlueprintCallable, Category = "Editor Utility Test")
	virtual void AddInfo(const FString& Message);

	/** Add error if expected condition is false */
	UFUNCTION(BlueprintCallable, Category = "Editor Utility Test", meta = (AdvancedDisplay = "ErrorMessage"))
	virtual void ExpectTrue(bool Condition, const FString& ErrorMessage);

	/** Add error if expected condition is true */
	UFUNCTION(BlueprintCallable, Category = "Editor Utility Test", meta = (AdvancedDisplay = "ErrorMessage"))
	virtual void ExpectFalse(bool Condition, const FString& ErrorMessage);

	virtual bool Tick(float DeltaSeconds);

	EEditorUtilityTestResult State;

private:
	/** Delegate for callbacks to Tick */
	FTickerDelegate TickDelegate;

	/** Handle to various registered delegates */
	FTSTicker::FDelegateHandle TickDelegateHandle;

	FDateTime StartTime;

public:
	/** Get test state */
	UFUNCTION(BlueprintCallable, Category = "Editor Utility Test")
	virtual EEditorUtilityTestResult GetState();

	/** Is test ruuning */
	UFUNCTION(BlueprintCallable, Category = "Editor Utility Test")
	virtual bool IsRunning();

};
