// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class EditorTests : ModuleRules
{
	public EditorTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				// ... add other public dependencies that you statically link with here ...
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Projects",
				"ApplicationCore",
				"InputCore",
				"LevelEditor",
				"CoreUObject",
                "RenderCore",
				"Engine",
                "NavigationSystem",
                "Slate",
				"SlateCore",
				"AssetTools",
				"MainFrame",
				"MaterialEditor",
				"JsonUtilitiesEditor",
				"JsonUtilities",
				"Json",
				"Analytics",
				"ContentBrowser",				
				"SourceControl",
				"RHI",
				"BlueprintGraph",
				"AddContentDialog",
				"GraphEditor",
				"DirectoryWatcher",
				"Projects",
				"EditorFramework",
				"UnrealEd",
				"AudioEditor",
				"AnimGraphRuntime",
				"MeshMergeUtilities",
				"MaterialBaking",
                "MeshDescription",
				"StaticMeshDescription",
                "MeshBuilder",
                "RawMesh",
				"AutomationController",
				"Blutility",
				"UMGEditor",
				"UMG"
			}
		);
	}
}
