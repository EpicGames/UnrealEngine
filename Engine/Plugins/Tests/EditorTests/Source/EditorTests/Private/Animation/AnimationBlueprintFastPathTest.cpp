// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimClassInterface.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimNode_Root.h"

DEFINE_LOG_CATEGORY_STATIC(LogAnimBlueprintFastPathTests, Log, All);

struct FCheckFastPathCommandPayload
{
	/** The filename to check */
	FString Filename;

	/** Whether to check enabled or disabled */
	bool bCheckEnabled;
};

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FCompileAnimBlueprintLatentCommand, FString, Filename);

bool FCompileAnimBlueprintLatentCommand::Update()
{
	UAnimBlueprint* AnimBlueprint = LoadObject<UAnimBlueprint>(nullptr, *Filename);
	if (AnimBlueprint != nullptr)
	{
		FKismetEditorUtilities::CompileBlueprint(AnimBlueprint);
		if (AnimBlueprint->Status == EBlueprintStatus::BS_UpToDate)
		{
			UE_LOGF(LogAnimBlueprintFastPathTests, Display, "Anim blueprint compiled successfully (%ls)", *AnimBlueprint->GetName());
		}
		else
		{
			UE_LOGF(LogAnimBlueprintFastPathTests, Error, "Anim blueprint is in an unexpected state after compiling (%ls)", *AnimBlueprint->GetName());
		}
	}

	return true;
}

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FCheckFastPathLatentCommand, FCheckFastPathCommandPayload, Payload);

bool FCheckFastPathLatentCommand::Update()
{
	UAnimBlueprint* AnimBlueprint = LoadObject<UAnimBlueprint>(nullptr, *Payload.Filename);
	if (AnimBlueprint != nullptr)
	{
		if (AnimBlueprint->GeneratedClass)
		{
			UAnimInstance* DefaultAnimInstance = Cast<UAnimInstance>(AnimBlueprint->GeneratedClass->GetDefaultObject());
			IAnimClassInterface* AnimClassInterface = Cast<IAnimClassInterface>(AnimBlueprint->GeneratedClass);
			if (AnimClassInterface && DefaultAnimInstance)
			{
				for (const FStructPropertyPath& Property : AnimClassInterface->GetAnimNodeProperties())
				{
					if (Property->Struct->IsChildOf(FAnimNode_Base::StaticStruct()) && !Property->Struct->IsChildOf(FAnimNode_Root::StaticStruct()))
					{
						FAnimNode_Base* AnimNode = Property->ContainerPtrToValuePtr<FAnimNode_Base>(DefaultAnimInstance);
						if (AnimNode)
						{
							if (AnimNode->GetEvaluateGraphExposedInputs().GetHandlerStruct() == FAnimNodeExposedValueHandler_PropertyAccess::StaticStruct())
							{
								const FAnimNodeExposedValueHandler_PropertyAccess* Handler = static_cast<const FAnimNodeExposedValueHandler_PropertyAccess*>(AnimNode->GetEvaluateGraphExposedInputs().GetHandler());

								if (Payload.bCheckEnabled)
								{
									if (Handler->BoundFunction != NAME_None)
									{
										UE_LOGF(LogAnimBlueprintFastPathTests, Error, "Found blueprint VM call (%ls) in fast-path Anim Blueprint (%ls)", *Handler->BoundFunction.ToString(), *AnimBlueprint->GetName());
									}
									for (const FExposedValueCopyRecord& CopyRecord : Handler->CopyRecords)
									{
										if (CopyRecord.CopyIndex == INDEX_NONE)
										{
											UE_LOGF(LogAnimBlueprintFastPathTests, Error, "Anim blueprint has an invalid copy index (%ls)", *AnimBlueprint->GetName());
										}
									}
								}
								else
								{
									if (Handler->BoundFunction == NAME_None)
									{
										UE_LOGF(LogAnimBlueprintFastPathTests, Error, "No function bound for node evaluation (%ls)", *AnimBlueprint->GetName());
									}
									if (Handler->CopyRecords.Num() > 0)
									{
										UE_LOGF(LogAnimBlueprintFastPathTests, Error, "Found copy records in non-fast-path node evaluator (%ls)", *AnimBlueprint->GetName());
									}
								}
							}
						}
					}
				}
			}
			else
			{
				UE_LOGF(LogAnimBlueprintFastPathTests, Error, "Anim blueprint has an invalid generated class or CDO (%ls)", *AnimBlueprint->GetName());
			}
		}
		else
		{
			UE_LOGF(LogAnimBlueprintFastPathTests, Error, "Anim blueprint has no generated class (%ls)", *AnimBlueprint->GetName());
		}
	}

	return true;
}

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FVerifyFastPathTest, "System.Engine.Animation.Verify Fast Path", (EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter))

void FVerifyFastPathTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	OutTestCommands.Add(TEXT("/Game/Tests/Animation/FastPath/SingleIntegralType.SingleIntegralType"));
	OutBeautifiedNames.Add(TEXT("IntegralType"));

	OutTestCommands.Add(TEXT("/Game/Tests/Animation/FastPath/MultiIntegralType.MultiIntegralType"));
	OutBeautifiedNames.Add(TEXT("MultiIntegralType"));

	OutTestCommands.Add(TEXT("/Game/Tests/Animation/FastPath/Struct.Struct"));
	OutBeautifiedNames.Add(TEXT("Struct"));

	OutTestCommands.Add(TEXT("/Game/Tests/Animation/FastPath/MultiStruct.MultiStruct"));
	OutBeautifiedNames.Add(TEXT("MultiStruct"));

	OutTestCommands.Add(TEXT("/Game/Tests/Animation/FastPath/SubStructBreak.SubStructBreak"));
	OutBeautifiedNames.Add(TEXT("SubStructBreak"));

	OutTestCommands.Add(TEXT("/Game/Tests/Animation/FastPath/SubStructSplit.SubStructSplit"));
	OutBeautifiedNames.Add(TEXT("SubStructSplit"));

	OutTestCommands.Add(TEXT("/Game/Tests/Animation/FastPath/NegatedBool.NegatedBool"));
	OutBeautifiedNames.Add(TEXT("NegatedBool"));

	OutTestCommands.Add(TEXT("/Game/Tests/Animation/FastPath/MultiPropertyToArray.MultiPropertyToArray"));
	OutBeautifiedNames.Add(TEXT("MultiPropertyToArray"));
}

bool FVerifyFastPathTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FCompileAnimBlueprintLatentCommand(Parameters));
	ADD_LATENT_AUTOMATION_COMMAND(FCheckFastPathLatentCommand({ Parameters, true }));

	return true;
}

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FVerifyNotFastPathTest, "System.Engine.Animation.Verify Not Fast Path", (EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter))

void FVerifyNotFastPathTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	OutTestCommands.Add(TEXT("/Game/Tests/Animation/FastPath/BlueprintLogic.BlueprintLogic"));
	OutBeautifiedNames.Add(TEXT("BlueprintLogic"));

	OutTestCommands.Add(TEXT("/Game/Tests/Animation/FastPath/NativeFunctionCall.NativeFunctionCall"));
	OutBeautifiedNames.Add(TEXT("NativeFunctionCall"));

	OutTestCommands.Add(TEXT("/Game/Tests/Animation/FastPath/BlueprintFunctionCall.BlueprintFunctionCall"));
	OutBeautifiedNames.Add(TEXT("BlueprintFunctionCall"));
}

bool FVerifyNotFastPathTest::RunTest(const FString& Parameters)
{
	ADD_LATENT_AUTOMATION_COMMAND(FCompileAnimBlueprintLatentCommand(Parameters));
	ADD_LATENT_AUTOMATION_COMMAND(FCheckFastPathLatentCommand({ Parameters, false }));

	return true;
}
