// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorTests.h"
#include "Modules/ModuleManager.h"

static const FName EditorTestsTabName("EditorTests");

#define LOCTEXT_NAMESPACE "FEditorTestsModule"

void FEditorTestsModule::StartupModule()
{
}

void FEditorTestsModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FEditorTestsModule, EditorTests)
