// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilityTest.h"
#include "Misc/AutomationTest.h"
#include "EditorUtilityBlueprint.h"
#include "EditorUtilitySubsystem.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "IAutomationControllerModule.h"
#include "Misc/Paths.h"
#include "Engine/Engine.h"
#include "UnrealEdMisc.h"

/////////////////////////////////////////////////////

DEFINE_LOG_CATEGORY_STATIC(LogEditorUtilityTest, Log, All);

class FBlueprintEditorUtilitiesTestBase : public FAutomationTestBase
{
public:
	FBlueprintEditorUtilitiesTestBase(const FString& InName, const bool bInComplexTask)
		: FAutomationTestBase(InName, bInComplexTask)
		, EditorUtility(nullptr)
	{
	}

	void SetEditorUtility(UEditorUtilityTest* InEditorUtility)
	{
		EditorUtility = InEditorUtility;
	}

	UEditorUtilityTest* GetEditorUtility()
	{
		return EditorUtility;
	}

	virtual FString GetTestOpenCommand(const FString& BlueprintAssetPath) const override
	{
		return FString::Printf(TEXT("Automate.OpenMap %s"), *BlueprintAssetPath);
	}

private:
	UEditorUtilityTest* EditorUtility;
};

inline FBlueprintEditorUtilitiesTestBase* GetTest()
{
	return static_cast<FBlueprintEditorUtilitiesTestBase*>(FAutomationTestFramework::Get().GetCurrentTest());
}

UEditorUtilityTest::UEditorUtilityTest(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, PreparationTimeLimit(15.0f)
	, TimeLimit(60.0f)
	, State(EEditorUtilityTestResult::Invalid)
	, StartTime(0)
{
	TickDelegate = FTickerDelegate::CreateUObject(this, &UEditorUtilityTest::Tick);
}

void UEditorUtilityTest::Run()
{
	if (State != EEditorUtilityTestResult::Preparing && State != EEditorUtilityTestResult::Running)
	{
		State = EEditorUtilityTestResult::Preparing;

		StartTime = FDateTime::Now();
		if (PreparationTimeLimit || TimeLimit)
		{
			TickDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(TickDelegate, 1.f);
		}

		FBlueprintEditorUtilitiesTestBase* CurrentTest = GetTest();
		if (CurrentTest != nullptr)
		{
			CurrentTest->SetEditorUtility(this);
		}

		if (FAutomationTestFramework::NeedLogBPTestMetadata() && GIsAutomationTesting)
		{
			AddInfo(FString::Printf(TEXT("[Owner] %s"), *Owner));
			AddInfo(FString::Printf(TEXT("[Description] %s"), *Description));
		}

		PrepareTest();
		OnTestPrepare.Broadcast();
	}
}

void UEditorUtilityTest::PrepareTest_Implementation()
{
	FinishPrepareTest();
}

void UEditorUtilityTest::FinishPrepareTest()
{
	if (State == EEditorUtilityTestResult::Preparing)
	{
		State = EEditorUtilityTestResult::Running;
		StartTest();
		OnTestStart.Broadcast();
	}
}

void UEditorUtilityTest::StartTest_Implementation()
{
	FinishTest(EEditorUtilityTestResult::Default, TEXT(""));
}

void UEditorUtilityTest::FinishTest(EEditorUtilityTestResult ProposedState, const FString& Message)
{
	if (State == EEditorUtilityTestResult::Running)
	{
		if (TickDelegateHandle.IsValid())
		{
			TickDelegateHandle.Reset();
		}

		if (ProposedState > EEditorUtilityTestResult::Running)
		{
			State = ProposedState;
		}
		else
		{
			State = EEditorUtilityTestResult::Succeeded;
		}

		EEditorUtilityTestResult NewState = EEditorUtilityTestResult::Default;
		ReceiveFinishedTest(State, NewState);
		if (NewState > EEditorUtilityTestResult::Running)
		{
			State = NewState;
		}

		if (State == EEditorUtilityTestResult::Succeeded && !Message.IsEmpty())
		{
			AddInfo(Message);
		}
		else if (State == EEditorUtilityTestResult::Failed)
		{
			AddError(Message.IsEmpty() ? TEXT("Test finished with a Failed state.") : Message);
		}

		FBlueprintEditorUtilitiesTestBase* CurrentTest = GetTest();
		if (CurrentTest != nullptr && CurrentTest->GetEditorUtility() == this)
		{
			// Reset blueprint test object tracking
			CurrentTest->SetEditorUtility(nullptr);
		}

		OnTestFinished.Broadcast(State);

		// Force GC at the end of every test.
		GEngine->ForceGarbageCollection();
	}
}

EEditorUtilityTestResult UEditorUtilityTest::GetState()
{
	return State;
}

bool UEditorUtilityTest::IsRunning()
{
	return State == EEditorUtilityTestResult::Preparing || State == EEditorUtilityTestResult::Running;
}

bool UEditorUtilityTest::Tick(float DeltaSeconds)
{
	float RunTime = (FDateTime::Now() - StartTime).GetTotalSeconds();
	if ((State == EEditorUtilityTestResult::Preparing && PreparationTimeLimit && RunTime > PreparationTimeLimit)
		|| (State == EEditorUtilityTestResult::Running && TimeLimit && RunTime > TimeLimit))
	{
		FString Message = FString::Printf(TEXT("Test reached timeout after %.01f seconds during %s phase."), RunTime, *UEnum::GetDisplayValueAsText(State).ToString());
		FinishTest(EEditorUtilityTestResult::Failed, Message);
	}

	return true;
}

void UEditorUtilityTest::AddError(const FString& Message)
{
	FBlueprintEditorUtilitiesTestBase* CurrentTest = GetTest();
	if (CurrentTest != nullptr && CurrentTest->GetEditorUtility() == this)
	{
		CurrentTest->AddError(Message);
	}
	else
	{
		UE_LOGF(LogEditorUtilityTest, Error, "%ls", *Message);
	}
}

void UEditorUtilityTest::AddWarning(const FString& Message)
{
	FBlueprintEditorUtilitiesTestBase* CurrentTest = GetTest();
	if (CurrentTest != nullptr && CurrentTest->GetEditorUtility() == this)
	{
		CurrentTest->AddWarning(Message);
	}
	else
	{
		UE_LOGF(LogEditorUtilityTest, Warning, "%ls", *Message);
	}
}

void UEditorUtilityTest::AddInfo(const FString& Message)
{
	FBlueprintEditorUtilitiesTestBase* CurrentTest = GetTest();
	if (CurrentTest != nullptr && CurrentTest->GetEditorUtility() == this)
	{
		CurrentTest->AddInfo(Message);
	}
	else
	{
		UE_LOGF(LogEditorUtilityTest, Log, "%ls", *Message);
	}
}

void UEditorUtilityTest::ExpectTrue(bool Condition, const FString& ErrorMessage)
{
	if (!Condition)
	{
		AddError(ErrorMessage.IsEmpty() ? TEXT("Expected condition was not True.") : ErrorMessage);
	}
}

void UEditorUtilityTest::ExpectFalse(bool Condition, const FString& ErrorMessage)
{
	if (Condition)
	{
		AddError(ErrorMessage.IsEmpty()? TEXT("Expected condition was not False.") : ErrorMessage);
	}
}

class FWaitForBlueprintEditorUtilityTest : public IAutomationLatentCommand
{
public:
	FWaitForBlueprintEditorUtilityTest()
	{}
	virtual bool Update() override
	{
		FBlueprintEditorUtilitiesTestBase* CurrentTest = GetTest();
		if (CurrentTest == nullptr)
		{
			return true;
		}
		UEditorUtilityTest* EditorUtility = CurrentTest->GetEditorUtility();

		if (EditorUtility == nullptr)
		{
			return true;
		}

		return !EditorUtility->IsRunning();
	}
};

IMPLEMENT_CUSTOM_COMPLEX_AUTOMATION_TEST(FBlueprintEditorUtilitiesTest, FBlueprintEditorUtilitiesTestBase, "Project.Blueprints.EditorUtilities", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter)

/**
 * Gather the tests to run
 */
void FBlueprintEditorUtilitiesTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	IAutomationControllerModule& AutomationControllerModule = FModuleManager::LoadModuleChecked<IAutomationControllerModule>(TEXT("AutomationController"));
	IAutomationControllerManagerPtr AutomationController = AutomationControllerModule.GetAutomationController();
	bool IsDeveloperDirectoryIncluded = AutomationController->IsDeveloperDirectoryIncluded();

	FTopLevelAssetPath BaseClassPathName = UEditorUtilityTest::StaticClass()->GetClassPathName();
	UClass* BlueprintClass = UEditorUtilityBlueprint::StaticClass();

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Use the asset registry to get the set of all class names deriving from Base
	TSet< FTopLevelAssetPath > DerivedPathNames;
	{
		TArray< FTopLevelAssetPath > BasePathNames;
		BasePathNames.Add(BaseClassPathName);

		TSet< FTopLevelAssetPath > Excluded;
		AssetRegistryModule.Get().GetDerivedClassNames(BasePathNames, Excluded, DerivedPathNames);
	}

	TArray<FAssetData> ObjectList;
	AssetRegistryModule.Get().GetAssetsByClass(BlueprintClass->GetClassPathName(), ObjectList);
	for (auto ObjIter = ObjectList.CreateConstIterator(); ObjIter; ++ObjIter)
	{
		const FAssetData& Asset = *ObjIter;
		FString Filename = Asset.GetObjectPathString();

		// Get the the class this blueprint generates
		auto AssetTag = Asset.TagsAndValues.FindTag(TEXT("GeneratedClass"));
		if (AssetTag.IsSet())
		{
			// Convert path to just the name part
			const FTopLevelAssetPath ClassPathName(FPackageName::ExportTextPathToObjectPath(*AssetTag.AsString()));

			// Check if this class is in the derived set
			if (!DerivedPathNames.Contains(ClassPathName))
			{
				continue;
			}

			// Exclude Developer folder
			if (!IsDeveloperDirectoryIncluded && Filename.Find(TEXT("/Game/Developers")) == 0) continue;

			// Extract folder org
			// Trim /Game/ otherwise use the plugin name as category 
			int32 TrimLength = Filename.Find(TEXT("/Game/")) == 0 ? 6 : 1;
			Filename = FPaths::GetPath(Filename.RightChop(TrimLength)).Replace(TEXT("/"), TEXT("."));
			FString BeautifiedFilename = Filename + TEXT(".") + Asset.AssetName.ToString();

			OutBeautifiedNames.Add(BeautifiedFilename);
			OutTestCommands.Add(Asset.GetObjectPathString());
		}
	}
}

/**
 * Runs Blueprint Editor Utilities Tests
 */
bool FBlueprintEditorUtilitiesTest::RunTest(const FString& BlueprintAssetPath)
{
	UObject* Object = StaticLoadObject(UObject::StaticClass(), NULL, *BlueprintAssetPath);

	UEditorUtilitySubsystem* EditorUtilitySubsystem = GEditor->GetEditorSubsystem<UEditorUtilitySubsystem>();
	EditorUtilitySubsystem->TryRun(Object);

	ADD_LATENT_AUTOMATION_COMMAND(FWaitForBlueprintEditorUtilityTest());

	return true;
}
