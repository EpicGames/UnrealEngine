// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/AutomationTest.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "Engine/Texture2D.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlRevision.h"
#include "ISourceControlModule.h"
#include "SourceControlHelpers.h"
#include "UObject/SavePackage.h"

#include "ISourceControlLabel.h"
#include "PackageTools.h"

#include "Tests/SourceControlAutomationCommon.h"

static void GetProviders(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands)
{
	ISourceControlModule& SourceControlModule = FModuleManager::LoadModuleChecked<ISourceControlModule>( "SourceControl" );

	TArray<FName> ProviderNames;
	SourceControlModule.GetProviderNames(ProviderNames);

	for ( FName ProviderName : ProviderNames )
	{
		if ( ProviderName != TEXT("None") )
		{
			OutBeautifiedNames.Add(ProviderName.ToString());
		}
	}

	// commands are the same as names in this case
	OutTestCommands = OutBeautifiedNames;
}

static void AppendFilename(const FString& InFilename, TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands)
{
	// append the filename to the commands we have passed in
	for(auto Iter(OutBeautifiedNames.CreateIterator()); Iter; Iter++)
	{
		*Iter = FString::Printf(TEXT("%s (%s)"), *InFilename, **Iter);
	}

	for(auto Iter(OutTestCommands.CreateIterator()); Iter; Iter++)
	{
		*Iter += TEXT(" ");
		*Iter += InFilename;
	}
}

/**
 * Helper struct used to restore read-only state
 */
struct FReadOnlyState
{
	FReadOnlyState(const FString& InPackageName, bool bInReadOnly)
		: PackageName(InPackageName)
		, bReadOnly(bInReadOnly)
	{
	}

	FString PackageName;

	bool bReadOnly;
};

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FSetReadOnlyFlag, FReadOnlyState, ReadOnlyState);

bool FSetReadOnlyFlag::Update()
{
	FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*SourceControlHelpers::PackageFilename(ReadOnlyState.PackageName), ReadOnlyState.bReadOnly);
	return true;
}

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FSetProviderLatentCommand, FName, ProviderName);

bool FSetProviderLatentCommand::Update()
{
	// set to 'None' first so the provider is reinitialized
	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
	SourceControlModule.SetProvider("None");
	SourceControlModule.SetProvider(ProviderName);
	if(SourceControlModule.GetProvider().GetName() != ProviderName || !SourceControlModule.IsEnabled())
	{
		UE_LOGF(LogSourceControl, Error, "Could not set provider to '%ls'", *ProviderName.ToString());
	}
	return true;
}

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FSetProviderTest, "Project.Editor.Revision Control.Set Provider", ( EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::RequiresUser | EAutomationTestFlags::EngineFilter))

void FSetProviderTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	GetProviders(OutBeautifiedNames, OutTestCommands);
}

bool FSetProviderTest::RunTest(const FString& Parameters)
{
	// parameter is the provider we want to use
	ADD_LATENT_AUTOMATION_COMMAND(FSetProviderLatentCommand(FName(*Parameters)));
	return true;
}

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FConnectLatentCommand, SourceControlAutomationCommon::FAsyncCommandHelper, AsyncHelper);

bool FConnectLatentCommand::Update()
{
	// attempt a login and wait for the result
	if(!AsyncHelper.IsDispatched())
	{
		if(ISourceControlModule::Get().GetProvider().Login( FString(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateRaw( &AsyncHelper, &SourceControlAutomationCommon::FAsyncCommandHelper::SourceControlOperationComplete ) ) != ECommandResult::Succeeded)
		{
			return false;
		}
		AsyncHelper.SetDispatched();
	}

	return AsyncHelper.IsDone();
}

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FConnectTest, "Project.Editor.Revision Control.Connect", ( EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::RequiresUser | EAutomationTestFlags::EngineFilter))

void FConnectTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	GetProviders(OutBeautifiedNames, OutTestCommands);
}

bool FConnectTest::RunTest(const FString& Parameters)
{
	// parameter is the provider we want to use
	ADD_LATENT_AUTOMATION_COMMAND(FSetProviderLatentCommand(FName(*Parameters)));
	ADD_LATENT_AUTOMATION_COMMAND(FConnectLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper()));

	return true;
}

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FRevertLatentCommand, SourceControlAutomationCommon::FAsyncCommandHelper, AsyncHelper);

bool FRevertLatentCommand::Update()
{
	if(!AsyncHelper.IsDispatched())
	{
		if( ISourceControlModule::Get().GetProvider().Execute( 
			ISourceControlOperation::Create<FRevert>(), 
			SourceControlHelpers::PackageFilename(AsyncHelper.GetParameter()),
			EConcurrency::Asynchronous, 
			FSourceControlOperationComplete::CreateRaw( &AsyncHelper, &SourceControlAutomationCommon::FAsyncCommandHelper::SourceControlOperationComplete ) 
			) != ECommandResult::Succeeded)
		{
			return true;
		}

		AsyncHelper.SetDispatched();
	}

	if(AsyncHelper.IsDone())
	{		
		// check state now we are done
		TSharedPtr<ISourceControlState, ESPMode::ThreadSafe> SourceControlState = ISourceControlModule::Get().GetProvider().GetState(SourceControlHelpers::PackageFilename(AsyncHelper.GetParameter()), EStateCacheUsage::Use);
		if(!SourceControlState.IsValid())
		{
			UE_LOGF(LogSourceControl, Error, "Could not retrieve state for file '%ls'", *AsyncHelper.GetParameter());
		}
		else
		{
			if(SourceControlState->IsSourceControlled())
			{
				if(!SourceControlState->CanCheckout())
				{
					UE_LOGF(LogSourceControl, Error, "Unexpected state following Revert operation for file '%ls'", *AsyncHelper.GetParameter());
				}
			}
		}
	}

	return AsyncHelper.IsDone();
}

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FCheckOutLatentCommand, SourceControlAutomationCommon::FAsyncCommandHelper, AsyncHelper);

bool FCheckOutLatentCommand::Update()
{
	if(!AsyncHelper.IsDispatched())
	{
		if( ISourceControlModule::Get().GetProvider().Execute( 
			ISourceControlOperation::Create<FCheckOut>(), 
			SourceControlHelpers::PackageFilename(AsyncHelper.GetParameter()),
			EConcurrency::Asynchronous, 
			FSourceControlOperationComplete::CreateRaw( &AsyncHelper, &SourceControlAutomationCommon::FAsyncCommandHelper::SourceControlOperationComplete ) 
			) != ECommandResult::Succeeded)
		{
			return true;
		}

		AsyncHelper.SetDispatched();
	}

	if(AsyncHelper.IsDone())
	{
		// check state now we are done
		TSharedPtr<ISourceControlState, ESPMode::ThreadSafe> SourceControlState = ISourceControlModule::Get().GetProvider().GetState(SourceControlHelpers::PackageFilename(AsyncHelper.GetParameter()), EStateCacheUsage::Use);
		if(!SourceControlState.IsValid())
		{
			UE_LOGF(LogSourceControl, Error, "Could not retrieve state for file '%ls'", *AsyncHelper.GetParameter());
		}
		else
		{
			if(!SourceControlState->IsCheckedOut())
			{
				UE_LOGF(LogSourceControl, Error, "Unexpected state following Check Out operation for file '%ls'", *AsyncHelper.GetParameter());
			}
		}
	}

	return AsyncHelper.IsDone();
}

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FCheckOutTest, "Project.Editor.Revision Control.Check Out", ( EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::RequiresUser | EAutomationTestFlags::EngineFilter))

void FCheckOutTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	GetProviders(OutBeautifiedNames, OutTestCommands);
	AppendFilename(TEXT("/Engine/NotForLicensees/Automation/EditorAutomation/SourceControlTest"), OutBeautifiedNames, OutTestCommands);
}

bool FCheckOutTest::RunTest(const FString& Parameters)
{
	// parameter is the provider we want to use followed by the filename
	const FString Delimiter(TEXT(" "));
	TArray<FString> ParamArray;
	Parameters.ParseIntoArray(ParamArray, *Delimiter, true);
	ensure(ParamArray.Num() == 2);

	// check to see if we should restore the read only status after this test
	bool bWasReadOnly = IFileManager::Get().IsReadOnly(*SourceControlHelpers::PackageFilename(ParamArray[1]));

	ADD_LATENT_AUTOMATION_COMMAND(FSetProviderLatentCommand(FName(*ParamArray[0])));
	ADD_LATENT_AUTOMATION_COMMAND(FConnectLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper()));
	ADD_LATENT_AUTOMATION_COMMAND(FCheckOutLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper(ParamArray[1])));
	ADD_LATENT_AUTOMATION_COMMAND(FRevertLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper(ParamArray[1])));

	ADD_LATENT_AUTOMATION_COMMAND(FSetReadOnlyFlag(FReadOnlyState(ParamArray[1], bWasReadOnly)));

	return true;
}

DECLARE_DELEGATE_OneParam(FAddLatentCommands, const FString&)

/**
 * Helper function used to generate parameters from one latent command to pass to another
 */
struct FLatentCommandChain
{
public:
	FLatentCommandChain(const FString& InParameter, const FAddLatentCommands& InLatentCommandsDelegate)
		: Parameter(InParameter)
		, LatentCommandDelegate(InLatentCommandsDelegate)
	{
	}

	/** Parameter to the first latent command */
	FString Parameter;

	/** Delegate to call once the first command is done (usually with output from the first latent command) */
	FAddLatentCommands LatentCommandDelegate;
};

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FCreatePackageLatentCommand, FLatentCommandChain, CommandChain);

bool FCreatePackageLatentCommand::Update()
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
	FString PackageName;
	FString AssetName;
	AssetToolsModule.Get().CreateUniqueAssetName(CommandChain.Parameter, TEXT("New"), PackageName, AssetName);
	
	FString OriginalPackageName = SourceControlHelpers::PackageFilename(CommandChain.Parameter);
	FString NewPackageName = FPackageName::LongPackageNameToFilename(PackageName, FPackageName::GetAssetPackageExtension());

	if(FPlatformFileManager::Get().GetPlatformFile().CopyFile(*NewPackageName, *OriginalPackageName))
	{
		UPackage* Package = LoadPackage(NULL, *NewPackageName, LOAD_None);
		if(Package != NULL)
		{
			CommandChain.LatentCommandDelegate.ExecuteIfBound(PackageName);
		}
		else
		{
			UE_LOGF(LogSourceControl, Error, "Could not load temporary package '%ls'", *PackageName);
		}
	}
	else
	{
		UE_LOGF(LogSourceControl, Error, "Could not create temporary package to add '%ls'", *PackageName);
	}

	return true;
}

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FDeletePackageLatentCommand, FString, Parameter);

bool FDeletePackageLatentCommand::Update()
{
	UPackage* Package = FindPackage(NULL, *Parameter);
	if(Package != NULL)
	{
		TArray<UPackage*> Packages;
		Packages.Add(Package);
		if(UPackageTools::UnloadPackages(Packages))
		{
			FString PackageFileName = SourceControlHelpers::PackageFilename(Parameter);
			if(!FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*PackageFileName))
			{
				UE_LOGF(LogSourceControl, Error, "Could not delete temporary package '%ls'", *PackageFileName);
			}
		}
		else
		{
			UE_LOGF(LogSourceControl, Error, "Could not unload temporary package '%ls'", *Parameter);
		}
	}
	else
	{
		UE_LOGF(LogSourceControl, Error, "Could not find temporary package '%ls'", *Parameter);
	}

	return true;
}

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FMarkForAddLatentCommand, SourceControlAutomationCommon::FAsyncCommandHelper, AsyncHelper);

bool FMarkForAddLatentCommand::Update()
{
	if(!AsyncHelper.IsDispatched())
	{
		if( ISourceControlModule::Get().GetProvider().Execute( 
			ISourceControlOperation::Create<FMarkForAdd>(), 
			SourceControlHelpers::PackageFilename(AsyncHelper.GetParameter()),
			EConcurrency::Asynchronous, 
			FSourceControlOperationComplete::CreateRaw( &AsyncHelper, &SourceControlAutomationCommon::FAsyncCommandHelper::SourceControlOperationComplete ) 
			) != ECommandResult::Succeeded)
		{
			return true;
		}

		AsyncHelper.SetDispatched();
	}

	if(AsyncHelper.IsDone())
	{
		// check state now we are done
		TSharedPtr<ISourceControlState, ESPMode::ThreadSafe> SourceControlState = ISourceControlModule::Get().GetProvider().GetState(SourceControlHelpers::PackageFilename(AsyncHelper.GetParameter()), EStateCacheUsage::Use);
		if(!SourceControlState.IsValid())
		{
			UE_LOGF(LogSourceControl, Error, "Could not retrieve state for file '%ls'", *AsyncHelper.GetParameter());
		}
		else
		{
			if(!SourceControlState->IsAdded())
			{
				UE_LOGF(LogSourceControl, Error, "Unexpected state following Mark For Add operation for file '%ls'", *AsyncHelper.GetParameter());
			}
		}
	}

	return AsyncHelper.IsDone();
}

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FMarkForAddTest, "Project.Editor.Revision Control.Mark For Add", ( EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::RequiresUser | EAutomationTestFlags::EngineFilter))

void FMarkForAddTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	GetProviders(OutBeautifiedNames, OutTestCommands);
	AppendFilename(TEXT("/Engine/NotForLicensees/Automation/EditorAutomation/SourceControlTest"), OutBeautifiedNames, OutTestCommands);
}

bool FMarkForAddTest::RunTest(const FString& Parameters)
{
	// parameter is the provider we want to use followed by the filename
	const FString Delimiter(TEXT(" "));
	TArray<FString> ParamArray;
	Parameters.ParseIntoArray(ParamArray, *Delimiter, true);
	ensure(ParamArray.Num() == 2);

	ADD_LATENT_AUTOMATION_COMMAND(FSetProviderLatentCommand(FName(*ParamArray[0])));
	ADD_LATENT_AUTOMATION_COMMAND(FConnectLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper()));

	struct Local
	{
		static void AddDependentCommands(const FString& InParameter)
		{
			ADD_LATENT_AUTOMATION_COMMAND(FMarkForAddLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper(InParameter)));
			ADD_LATENT_AUTOMATION_COMMAND(FRevertLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper(InParameter)));
			ADD_LATENT_AUTOMATION_COMMAND(FDeletePackageLatentCommand(InParameter));
		}
	};

	ADD_LATENT_AUTOMATION_COMMAND(FCreatePackageLatentCommand(FLatentCommandChain(ParamArray[1], FAddLatentCommands::CreateStatic(&Local::AddDependentCommands))));

	return true;
}

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FDeleteLatentCommand, SourceControlAutomationCommon::FAsyncCommandHelper, AsyncHelper);

bool FDeleteLatentCommand::Update()
{
	if(!AsyncHelper.IsDispatched())
	{
		if( ISourceControlModule::Get().GetProvider().Execute( 
			ISourceControlOperation::Create<FDelete>(), 
			SourceControlHelpers::PackageFilename(AsyncHelper.GetParameter()),
			EConcurrency::Asynchronous, 
			FSourceControlOperationComplete::CreateRaw( &AsyncHelper, &SourceControlAutomationCommon::FAsyncCommandHelper::SourceControlOperationComplete ) 
			) != ECommandResult::Succeeded)
		{
			return true;
		}

		AsyncHelper.SetDispatched();
	}

	if(AsyncHelper.IsDone())
	{
		// check state now we are done
		TSharedPtr<ISourceControlState, ESPMode::ThreadSafe> SourceControlState = ISourceControlModule::Get().GetProvider().GetState(SourceControlHelpers::PackageFilename(AsyncHelper.GetParameter()), EStateCacheUsage::Use);
		if(!SourceControlState.IsValid())
		{
			UE_LOGF(LogSourceControl, Error, "Could not retrieve state for file '%ls'", *AsyncHelper.GetParameter());
		}
		else
		{
			if(!SourceControlState->IsDeleted())
			{
				UE_LOGF(LogSourceControl, Error, "Unexpected state following Delete operation for file '%ls'", *AsyncHelper.GetParameter());
			}
		}
	}

	return AsyncHelper.IsDone();
}

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FDeleteTest, "Project.Editor.Revision Control.Delete", ( EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::RequiresUser | EAutomationTestFlags::EngineFilter))

void FDeleteTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	GetProviders(OutBeautifiedNames, OutTestCommands);
	AppendFilename(TEXT("/Engine/NotForLicensees/Automation/EditorAutomation/SourceControlTest"), OutBeautifiedNames, OutTestCommands);
}

bool FDeleteTest::RunTest(const FString& Parameters)
{
	// parameter is the provider we want to use followed by the filename
	const FString Delimiter(TEXT(" "));
	TArray<FString> ParamArray;
	Parameters.ParseIntoArray(ParamArray, *Delimiter, true);
	ensure(ParamArray.Num() == 2);

	// check to see if we should restore the read only status after this test
	bool bWasReadOnly = IFileManager::Get().IsReadOnly(*SourceControlHelpers::PackageFilename(ParamArray[1]));

	FString AbsoluteFilename = SourceControlHelpers::PackageFilename(ParamArray[1]);

	ADD_LATENT_AUTOMATION_COMMAND(FSetProviderLatentCommand(FName(*ParamArray[0])));
	ADD_LATENT_AUTOMATION_COMMAND(FConnectLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper()));
	ADD_LATENT_AUTOMATION_COMMAND(FDeleteLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper(AbsoluteFilename)));
	ADD_LATENT_AUTOMATION_COMMAND(FRevertLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper(AbsoluteFilename)));

	ADD_LATENT_AUTOMATION_COMMAND(FSetReadOnlyFlag(FReadOnlyState(ParamArray[1], bWasReadOnly)));

	return true;
}

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FCheckInLatentCommand, SourceControlAutomationCommon::FAsyncCommandHelper, AsyncHelper);

bool FCheckInLatentCommand::Update()
{
	if(!AsyncHelper.IsDispatched())
	{
		TSharedRef<FCheckIn, ESPMode::ThreadSafe> CheckInOperation = ISourceControlOperation::Create<FCheckIn>();
		CheckInOperation->SetDescription(NSLOCTEXT("SourceControlTests", "TestChangelistDescription", "[AUTOMATED TEST] Automatic checkin, testing functionality."));

		if( ISourceControlModule::Get().GetProvider().Execute( 
			CheckInOperation, 
			SourceControlHelpers::PackageFilename(AsyncHelper.GetParameter()),
			EConcurrency::Asynchronous, 
			FSourceControlOperationComplete::CreateRaw( &AsyncHelper, &SourceControlAutomationCommon::FAsyncCommandHelper::SourceControlOperationComplete ) 
			) != ECommandResult::Succeeded)
		{
			return true;
		}

		AsyncHelper.SetDispatched();
	}

	if(AsyncHelper.IsDone())
	{
		// check state now we are done
		TSharedPtr<ISourceControlState, ESPMode::ThreadSafe> SourceControlState = ISourceControlModule::Get().GetProvider().GetState(SourceControlHelpers::PackageFilename(AsyncHelper.GetParameter()), EStateCacheUsage::Use);
		if(!SourceControlState.IsValid())
		{
			UE_LOGF(LogSourceControl, Error, "Could not retrieve state for file '%ls'", *AsyncHelper.GetParameter());
		}
		else
		{
			if(!SourceControlState->IsSourceControlled() || !SourceControlState->CanCheckout())
			{
				UE_LOGF(LogSourceControl, Error, "Unexpected state following Check In operation for file '%ls'", *AsyncHelper.GetParameter());
			}
		}
	}

	return AsyncHelper.IsDone();
}

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FEditTextureLatentCommand, FString, PackageName);

bool FEditTextureLatentCommand::Update()
{
	// make a minor edit to the texture in the package we are passed
	UPackage* Package = LoadPackage(NULL, *PackageName, LOAD_None);
	if(Package != NULL)
	{
		UTexture2D* Texture = FindObject<UTexture2D>(Package, TEXT("SourceControlTest"));
		check(Texture);
		Texture->AdjustBrightness = FMath::FRand();
		Package->SetDirtyFlag(true);
		FSavePackageArgs SaveArgs;
		SaveArgs.TopLevelFlags = RF_Standalone;
		SaveArgs.SaveFlags = SAVE_NoError;
		if(!UPackage::SavePackage(Package, nullptr,
			*FPackageName::LongPackageNameToFilename(PackageName, FPackageName::GetAssetPackageExtension()),
			SaveArgs))
		{
			UE_LOGF(LogSourceControl, Error, "Could not save package: '%ls'", *PackageName);
		}
		TArray<UPackage*> Packages;
		Packages.Add(Package);
		UPackageTools::UnloadPackages(Packages);
	}
	else
	{
		UE_LOGF(LogSourceControl, Error, "Could not find package for edit: '%ls'", *PackageName);
	}

	return true;
}

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FCheckInTest, "Project.Editor.Revision Control.Check In", ( EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::RequiresUser | EAutomationTestFlags::EngineFilter))

void FCheckInTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	GetProviders(OutBeautifiedNames, OutTestCommands);
	AppendFilename(TEXT("/Engine/NotForLicensees/Automation/EditorAutomation/SourceControlTest"), OutBeautifiedNames, OutTestCommands);
}

bool FCheckInTest::RunTest(const FString& Parameters)
{
	// parameter is the provider we want to use followed by the filename
	const FString Delimiter(TEXT(" "));
	TArray<FString> ParamArray;
	Parameters.ParseIntoArray(ParamArray, *Delimiter, true);
	ensure(ParamArray.Num() == 2);

	// check to see if we should restore the read only status after this test
	bool bWasReadOnly = IFileManager::Get().IsReadOnly(*SourceControlHelpers::PackageFilename(ParamArray[1]));

	// parameter is the provider we want to use
	ADD_LATENT_AUTOMATION_COMMAND(FSetProviderLatentCommand(FName(*ParamArray[0])));
	ADD_LATENT_AUTOMATION_COMMAND(FConnectLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper()));
	ADD_LATENT_AUTOMATION_COMMAND(FCheckOutLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper(ParamArray[1])));
	ADD_LATENT_AUTOMATION_COMMAND(FEditTextureLatentCommand(ParamArray[1]));
	ADD_LATENT_AUTOMATION_COMMAND(FCheckInLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper(ParamArray[1])));

	ADD_LATENT_AUTOMATION_COMMAND(FSetReadOnlyFlag(FReadOnlyState(ParamArray[1], bWasReadOnly)));

	return true;
}

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FSyncLatentCommand, SourceControlAutomationCommon::FAsyncCommandHelper, AsyncHelper);

bool FSyncLatentCommand::Update()
{
	if(!AsyncHelper.IsDispatched())
	{
		if( ISourceControlModule::Get().GetProvider().Execute( 
			ISourceControlOperation::Create<FSync>(), 
			SourceControlHelpers::PackageFilename(AsyncHelper.GetParameter()),
			EConcurrency::Asynchronous, 
			FSourceControlOperationComplete::CreateRaw( &AsyncHelper, &SourceControlAutomationCommon::FAsyncCommandHelper::SourceControlOperationComplete ) 
			) != ECommandResult::Succeeded)
		{
			return true;
		}

		AsyncHelper.SetDispatched();
	}

	if(AsyncHelper.IsDone())
	{
		// check state now we are done
		TSharedPtr<ISourceControlState, ESPMode::ThreadSafe> SourceControlState = ISourceControlModule::Get().GetProvider().GetState(SourceControlHelpers::PackageFilename(AsyncHelper.GetParameter()), EStateCacheUsage::Use);
		if(!SourceControlState.IsValid())
		{
			UE_LOGF(LogSourceControl, Error, "Could not retrieve state for file '%ls'", *AsyncHelper.GetParameter());
		}
		else
		{
			if(!SourceControlState->IsCurrent())
			{
				UE_LOGF(LogSourceControl, Error, "Unexpected state following Sync operation for file '%ls'", *AsyncHelper.GetParameter());
			}
		}
	}

	return AsyncHelper.IsDone();
}

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FSyncTest, "Project.Editor.Revision Control.Sync", ( EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::RequiresUser | EAutomationTestFlags::EngineFilter))

void FSyncTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	GetProviders(OutBeautifiedNames, OutTestCommands);
	AppendFilename(TEXT("/Engine/NotForLicensees/Automation/EditorAutomation/SourceControlTest"), OutBeautifiedNames, OutTestCommands);
}

bool FSyncTest::RunTest(const FString& Parameters)
{
	// parameter is the provider we want to use followed by the filename
	const FString Delimiter(TEXT(" "));
	TArray<FString> ParamArray;
	Parameters.ParseIntoArray(ParamArray, *Delimiter, true);
	ensure(ParamArray.Num() == 2);

	ADD_LATENT_AUTOMATION_COMMAND(FSetProviderLatentCommand(FName(*ParamArray[0])));
	ADD_LATENT_AUTOMATION_COMMAND(FConnectLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper()));
	ADD_LATENT_AUTOMATION_COMMAND(FSyncLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper(ParamArray[1])));

	return true;
}

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FRevertTest, "Project.Editor.Revision Control.Revert", ( EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::RequiresUser | EAutomationTestFlags::EngineFilter))

void FRevertTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	GetProviders(OutBeautifiedNames, OutTestCommands);
	AppendFilename(TEXT("/Engine/NotForLicensees/Automation/EditorAutomation/SourceControlTest"), OutBeautifiedNames, OutTestCommands);
}

bool FRevertTest::RunTest(const FString& Parameters)
{
	// parameter is the provider we want to use followed by the filename
	const FString Delimiter(TEXT(" "));
	TArray<FString> ParamArray;
	Parameters.ParseIntoArray(ParamArray, *Delimiter, true);
	ensure(ParamArray.Num() == 2);

	ADD_LATENT_AUTOMATION_COMMAND(FSetProviderLatentCommand(FName(*ParamArray[0])));
	ADD_LATENT_AUTOMATION_COMMAND(FConnectLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper()));

	struct Local
	{
		static void AddDepedentCommands(const FString& InParameter)
		{
			ADD_LATENT_AUTOMATION_COMMAND(FMarkForAddLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper(InParameter)));
			ADD_LATENT_AUTOMATION_COMMAND(FRevertLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper(InParameter)));
			ADD_LATENT_AUTOMATION_COMMAND(FDeletePackageLatentCommand(InParameter));
		}
	};

	ADD_LATENT_AUTOMATION_COMMAND(FCreatePackageLatentCommand(FLatentCommandChain(ParamArray[1], FAddLatentCommands::CreateStatic(&Local::AddDepedentCommands))))

	return true;
}

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FUpdateStatusLatentCommand, SourceControlAutomationCommon::FAsyncCommandHelper, AsyncHelper);

bool FUpdateStatusLatentCommand::Update()
{
	if(!AsyncHelper.IsDispatched())
	{
		TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> UpdateStatusOperation = ISourceControlOperation::Create<FUpdateStatus>();
		UpdateStatusOperation->SetUpdateHistory(true);
		UpdateStatusOperation->SetGetOpenedOnly(true);

		if( ISourceControlModule::Get().GetProvider().Execute( 
			UpdateStatusOperation, 
			SourceControlHelpers::PackageFilename(AsyncHelper.GetParameter()),
			EConcurrency::Asynchronous, 
			FSourceControlOperationComplete::CreateRaw( &AsyncHelper, &SourceControlAutomationCommon::FAsyncCommandHelper::SourceControlOperationComplete ) 
			) != ECommandResult::Succeeded)
		{
			return true;
		}

		AsyncHelper.SetDispatched();
	}

	return AsyncHelper.IsDone();
}

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FGetStateLatentCommand, FString, Filename);

bool FGetStateLatentCommand::Update()
{
	FSourceControlStatePtr SourceControlState = ISourceControlModule::Get().GetProvider().GetState(SourceControlHelpers::PackageFilename(Filename), EStateCacheUsage::Use);
	if(!SourceControlState.IsValid())
	{
		UE_LOGF(LogSourceControl, Error, "Failed to get a valid state for file: %ls", *Filename);
	}
	else
	{
		if(!SourceControlState->IsCheckedOut())
		{
			UE_LOGF(LogSourceControl, Error, "File '%ls' should be checked out, but isnt.", *Filename);
		}
		else
		{
			if(SourceControlState->GetHistorySize() == 0)
			{
				UE_LOGF(LogSourceControl, Error, "Failed to get a valid history for file: %ls", *Filename);
			}
			else
			{
				TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> HistoryItem = SourceControlState->GetHistoryItem(0);
				if(!HistoryItem.IsValid())
				{
					UE_LOGF(LogSourceControl, Error, "Failed to get a valid history item 0 for file: %ls", *Filename);
				}
			}
		}
	}

	return true;
}

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FUpdateStatusTest, "Project.Editor.Revision Control.Update Status", ( EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::RequiresUser | EAutomationTestFlags::EngineFilter))

void FUpdateStatusTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	GetProviders(OutBeautifiedNames, OutTestCommands);
	AppendFilename(TEXT("/Engine/NotForLicensees/Automation/EditorAutomation/SourceControlTest"), OutBeautifiedNames, OutTestCommands);
}

bool FUpdateStatusTest::RunTest(const FString& Parameters)
{
	// parameter is the provider we want to use followed by the filename
	const FString Delimiter(TEXT(" "));
	TArray<FString> ParamArray;
	Parameters.ParseIntoArray(ParamArray, *Delimiter, true);
	ensure(ParamArray.Num() == 2);

	ADD_LATENT_AUTOMATION_COMMAND(FSetProviderLatentCommand(FName(*ParamArray[0])));
	ADD_LATENT_AUTOMATION_COMMAND(FConnectLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper()));
	ADD_LATENT_AUTOMATION_COMMAND(FCheckOutLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper(ParamArray[1])));
	ADD_LATENT_AUTOMATION_COMMAND(FUpdateStatusLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper(ParamArray[1])));
	ADD_LATENT_AUTOMATION_COMMAND(FGetStateLatentCommand(ParamArray[1]));
	ADD_LATENT_AUTOMATION_COMMAND(FRevertLatentCommand(ParamArray[1]));

	return true;
}

/**
 * Helper struct for FGetLabelLatentCommand
 */
struct FLabelAndFilenames
{
	FLabelAndFilenames( const FString& InLabel, const TArray<FString>& InFilenames )
		: Label(InLabel)
		, Filenames(InFilenames)
	{
	}

	/** Label to use */
	FString Label;

	/** Filenames to use */
	TArray<FString> Filenames;
};

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FGetLabelLatentCommand, FLabelAndFilenames, LabelAndFilenames);

bool FGetLabelLatentCommand::Update()
{
	// @todo: for the moment, getting labels etc. is synchronous.

	TArray< TSharedRef<ISourceControlLabel> > Labels = ISourceControlModule::Get().GetProvider().GetLabels(LabelAndFilenames.Label);
	if(Labels.Num() == 0)
	{
		UE_LOGF(LogSourceControl, Error, "No labels available that use the spec '%ls'", *LabelAndFilenames.Label);
	}
	else
	{
		TArray< TSharedRef<ISourceControlRevision, ESPMode::ThreadSafe> > Revisions;
		Labels[0]->GetFileRevisions(LabelAndFilenames.Filenames[0], Revisions);
		if(Revisions.Num() == 0)
		{
			UE_LOGF(LogSourceControl, Error, "No revisions of file '%ls' found at label '%ls'", *LabelAndFilenames.Filenames[0], *LabelAndFilenames.Label);
		}
		else
		{
			FString TempGetFilename;
			Revisions[0]->Get(TempGetFilename);
			if(TempGetFilename.Len() == 0 || !FPaths::FileExists(TempGetFilename))
			{
				UE_LOGF(LogSourceControl, Error, "Could not get revision of file '%ls' using label '%ls'", *LabelAndFilenames.Filenames[0], *LabelAndFilenames.Label);
			}

			FString TempGetAnnotatedFilename;
			Revisions[0]->GetAnnotated(TempGetAnnotatedFilename);
			if(TempGetAnnotatedFilename.Len() == 0 || !FPaths::FileExists(TempGetAnnotatedFilename))
			{
				UE_LOGF(LogSourceControl, Error, "Could not get annotated revision of file '%ls' using label '%ls'", *LabelAndFilenames.Filenames[0], *LabelAndFilenames.Label);
			}
		}
	}

	return true;
}

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FGetLabelTest, "Project.Editor.Revision Control.Get Label", ( EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::RequiresUser | EAutomationTestFlags::EngineFilter))

void FGetLabelTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	GetProviders(OutBeautifiedNames, OutTestCommands);
	AppendFilename(TEXT("SourceControlAutomationLabel"), OutBeautifiedNames, OutTestCommands);
}

bool FGetLabelTest::RunTest(const FString& Parameters)
{
	// parameter is the provider we want to use followed by the label
	const FString Delimiter(TEXT(" "));
	TArray<FString> ParamArray;
	Parameters.ParseIntoArray(ParamArray, *Delimiter, true);
	ensure(ParamArray.Num() == 2);

	TArray<FString> FilesToGet;
	FilesToGet.Add(FPaths::ConvertRelativePathToFull(TEXT("../../../Engine/Source/Developer/SourceControl/SourceControl.Build.cs")));

	ADD_LATENT_AUTOMATION_COMMAND(FSetProviderLatentCommand(FName(*ParamArray[0])));
	ADD_LATENT_AUTOMATION_COMMAND(FConnectLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper()));
	ADD_LATENT_AUTOMATION_COMMAND(FGetLabelLatentCommand(FLabelAndFilenames(ParamArray[1], FilesToGet)));

	return true;
}

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FSyncLabelLatentCommand, FLabelAndFilenames, LabelAndFilenames);

bool FSyncLabelLatentCommand::Update()
{
	// @todo: for the moment, getting labels etc. is synchronous.

	TArray< TSharedRef<ISourceControlLabel> > Labels = ISourceControlModule::Get().GetProvider().GetLabels(LabelAndFilenames.Label);
	if(Labels.Num() == 0)
	{
		UE_LOGF(LogSourceControl, Error, "No labels available that use the spec '%ls'", *LabelAndFilenames.Label);
	}
	else
	{
		Labels[0]->Sync(LabelAndFilenames.Filenames);
	}

	return true;
}

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FSyncLabelTest, "Project.Editor.Revision Control.Sync Label", ( EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::RequiresUser | EAutomationTestFlags::EngineFilter))

void FSyncLabelTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	GetProviders(OutBeautifiedNames, OutTestCommands);
	AppendFilename(TEXT("SourceControlAutomationLabel"), OutBeautifiedNames, OutTestCommands);
}

bool FSyncLabelTest::RunTest(const FString& Parameters)
{
	// parameter is the provider we want to use followed by the label
	const FString Delimiter(TEXT(" "));
	TArray<FString> ParamArray;
	Parameters.ParseIntoArray(ParamArray, *Delimiter, true);
	ensure(ParamArray.Num() == 2);

	TArray<FString> FilesToGet;
	FilesToGet.Add(FPaths::ConvertRelativePathToFull(TEXT("../../../Engine/Source/Developer/SourceControl/SourceControl.Build.cs")));
	FilesToGet.Add(FPaths::ConvertRelativePathToFull(TEXT("../../../Engine/Source/Developer/SourceControl/Public/ISourceControlModule.h")));

	ADD_LATENT_AUTOMATION_COMMAND(FSetProviderLatentCommand(FName(*ParamArray[0])));
	ADD_LATENT_AUTOMATION_COMMAND(FConnectLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper()));
	ADD_LATENT_AUTOMATION_COMMAND(FSyncLabelLatentCommand(FLabelAndFilenames(ParamArray[1], FilesToGet)));

	return true;
}

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FGetRevisionLatentCommand, FString, Filename);

bool FGetRevisionLatentCommand::Update()
{
	// @todo: for the moment, getting revisions etc. is synchronous.

	FSourceControlStatePtr SourceControlState = ISourceControlModule::Get().GetProvider().GetState(SourceControlHelpers::PackageFilename(Filename), EStateCacheUsage::Use);
	if(!SourceControlState.IsValid())
	{
		UE_LOGF(LogSourceControl, Error, "Failed to get a valid state for file: %ls", *Filename);
	}
	else
	{
		if(SourceControlState->GetHistorySize() == 0)
		{
			UE_LOGF(LogSourceControl, Error, "Failed to get a valid history for file: %ls", *Filename);
		}
		else
		{
			TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> HistoryItem = SourceControlState->GetHistoryItem(0);
			if(!HistoryItem.IsValid())
			{
				UE_LOGF(LogSourceControl, Error, "Failed to get a valid history item 0 for file: %ls", *Filename);
			}
			else
			{
				FString TempGetFilename;
				HistoryItem->Get(TempGetFilename);
				if(TempGetFilename.Len() == 0 || !FPaths::FileExists(TempGetFilename))
				{
					UE_LOGF(LogSourceControl, Error, "Could not get revision of file '%ls'", *Filename);
				}
			}
		}
	}

	return true;
}

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FGetRevisionTest, "Project.Editor.Revision Control.Get Revision", ( EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::RequiresUser | EAutomationTestFlags::EngineFilter))

void FGetRevisionTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	GetProviders(OutBeautifiedNames, OutTestCommands);
	AppendFilename(TEXT("/Engine/NotForLicensees/Automation/EditorAutomation/SourceControlTest"), OutBeautifiedNames, OutTestCommands);
}

bool FGetRevisionTest::RunTest(const FString& Parameters)
{
	// parameter is the provider we want to use followed by the filename
	const FString Delimiter(TEXT(" "));
	TArray<FString> ParamArray;
	Parameters.ParseIntoArray(ParamArray, *Delimiter, true);
	ensure(ParamArray.Num() == 2);

	ADD_LATENT_AUTOMATION_COMMAND(FSetProviderLatentCommand(FName(*ParamArray[0])));
	ADD_LATENT_AUTOMATION_COMMAND(FConnectLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper()));
	ADD_LATENT_AUTOMATION_COMMAND(FUpdateStatusLatentCommand(SourceControlAutomationCommon::FAsyncCommandHelper(ParamArray[1])));
	ADD_LATENT_AUTOMATION_COMMAND(FGetRevisionLatentCommand(ParamArray[1]));

	return true;
}
