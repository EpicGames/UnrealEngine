// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/WildcardString.h"
#include "Misc/AutomationTest.h"

#include "FileCache.h"
#include "Interfaces/IPluginManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogAutoReimportTests, Log, All);

/** Simple latent copmmand that executes a function after a given delay */
class FDelayedCallbackLatentCommand : public IAutomationLatentCommand
{
public:
	FDelayedCallbackLatentCommand(TFunction<void()>&& InCallback, float InDelay = 0.1f)
		: Callback(MoveTemp(InCallback)), Delay(InDelay)
	{
		//Set the start time
		StartTime = FPlatformTime::Seconds();
	}

	virtual bool Update() override
	{
		double NewTime = FPlatformTime::Seconds();
		if (NewTime - StartTime >= static_cast<double>(Delay))
		{
			Callback();
			return true;
		}
		return false;
	}

private:
	TFunction<void()> Callback;
	float Delay;
};

/** Generate a config from the specified options, to pass to FFileCache on construction */
DirectoryWatcher::FFileCacheConfig GenerateFileCacheConfig(const FString& InWorkingDir)
{
	FString Directory = FPaths::ConvertRelativePathToFull(InWorkingDir);
	FString CacheFilename = Directory / TEXT("Cache.bin");

	DirectoryWatcher::FFileCacheConfig Config(Directory / TEXT("Content") / TEXT(""), MoveTemp(CacheFilename));
	// We always store paths inside content folders relative to the folder
	Config.PathType = DirectoryWatcher::EPathType::Relative;

	Config.bDetectChangesSinceLastRun = false;
	return Config;
}

/** Persistent test payload used for async testing, generally captured by lambdas */
struct FAutoReimportTestPayload
{
	DirectoryWatcher::FFileCacheConfig Config;
	TSharedPtr<DirectoryWatcher::FFileCache> FileCache;
	FString WorkingDir;

	FAutoReimportTestPayload(const FString& InWorkingDir)
		: Config(GenerateFileCacheConfig(InWorkingDir))
		, WorkingDir(InWorkingDir)
	{	
		if (!IFileManager::Get().MakeDirectory(*Config.Directory, true))
		{
			uint32 ErrorCode = FPlatformMisc::GetLastError();
			UE_LOGF(LogAutoReimportTests, Error, "Failed to create the test cache folder %ls. LastError code: %d", *Config.Directory, ErrorCode);
		}
		// Just to be sure...
		if (!IFileManager::Get().MakeDirectory(*InWorkingDir, true))
		{
			uint32 ErrorCode = FPlatformMisc::GetLastError();
			UE_LOGF(LogAutoReimportTests, Error, "Failed to create the test cache folder %ls (retry operation). LastError code: %d", *InWorkingDir, ErrorCode);
		}
	}
	~FAutoReimportTestPayload()
	{
		// Avoid writing out the file after we've deleted the directory
		FileCache = nullptr;
		if (!IFileManager::Get().DeleteDirectory(*WorkingDir, false, true))
		{
			UE_LOGF(LogAutoReimportTests, Error, "Failed to delete the test cache folder %ls.", *WorkingDir);
		}
	}

	void StartWatching()
	{
		if (!FileCache.IsValid())
		{
			FileCache = MakeShareable(new DirectoryWatcher::FFileCache(Config));
		}
	}

	void WaitForStartup(TFunction<void()>&& Finished)
	{
		// Ideally, this would capture-by-move, but we don't have full compiler support for that yet.
		ADD_LATENT_AUTOMATION_COMMAND(FDelayedCallbackLatentCommand([this, Finished]() mutable {
			FileCache->Tick();
			if (FileCache->MoveDetectionInitialized())
			{
				Finished();
			}
			else
			{
				WaitForStartup(MoveTemp(Finished));
			}
		}, 0.1f));
	}

	void StopWatching()
	{
		FileCache = nullptr;
	}
};

namespace AutoReimportTests
{
	FString GetWorkingDir()
	{
		return FPaths::ConvertRelativePathToFull(FPaths::AutomationTransientDir()) / TEXT("AutoReimport") / TEXT("");
	}
}

FString GetTestSourceDir()
{
	FString EditorTestsContent = IPluginManager::Get().FindPlugin(TEXT("EditorTests"))->GetContentDir();

	return FPaths::ConvertRelativePathToFull(EditorTestsContent) / TEXT("Editor") / TEXT("AutoReimport") / TEXT("");
}

struct FSrcDstFilenames
{
	FSrcDstFilenames(const TCHAR* InSrc, const TCHAR* InDst) : Src(InSrc), Dst(InDst) {}
	const TCHAR* Src, *Dst;
};

/** Copy an array of test files from the source folder to the transient test folder */
bool CopyTestFiles(const FAutoReimportTestPayload& Test, const TArray<FSrcDstFilenames>& Files)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	const FString SourceDir = GetTestSourceDir();

	for (const auto& File : Files)
	{
		const bool bCopied = PlatformFile.CopyFile(*(Test.Config.Directory / File.Dst), *(SourceDir / File.Src));
		if (!bCopied)
		{
			UE_LOGF(LogAutoReimportTests, Error, "Failed to copy source file %ls to the test directory as %ls.", File.Src, File.Dst);
			return false;
		}
	}

	return true;
}

const int32 LocalMaxRetryCounter = 5;
const float LocalRetryDelay = 5.0f;

void RetryLatentCommand(TFunction<bool(const bool)>&& LatentCommandFunction, const float RetryDelay, int32& MaxRetryCounter)
{
	ADD_LATENT_AUTOMATION_COMMAND(FDelayedCallbackLatentCommand([=]() mutable {
		if (!LatentCommandFunction(MaxRetryCounter == 1) && MaxRetryCounter > 0)
		{
			RetryLatentCommand(MoveTemp(LatentCommandFunction), RetryDelay, --MaxRetryCounter);
		}
	}, RetryDelay));
}

/** Test that creating a new file gets reported correctly */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoReimportSimpleCreateTest, "Editor.Auto Reimport.Simple Create", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAutoReimportSimpleCreateTest::RunTest(const FString& Parameters)
{
	const FString WorkingDir = AutoReimportTests::GetWorkingDir();

	static const TCHAR* Filename = TEXT("square.png");

	// Start watching the directory
	TSharedPtr<FAutoReimportTestPayload> Test = MakeShareable(new FAutoReimportTestPayload(WorkingDir));
	Test->StartWatching();
	Test->WaitForStartup([=]{

		TArray<FSrcDstFilenames> Files;
		Files.Emplace(Filename, Filename);

		if (!CopyTestFiles(*Test, Files))
		{
			return;
		}

		auto SimpleCreateFunction = [=](const bool bLastIteration)->bool
		{
			auto Changes = Test->FileCache->GetOutstandingChanges();
			if (!bLastIteration && Changes.Num() == 0)
			{
				return false;
			}

			if (Changes.Num() != 1)
			{
				UE_LOGF(LogAutoReimportTests, Error, "Incorrect number of changes reported (%d != 1).", Changes.Num());
			}
			// Check that we got a relative path
			else if (!Changes[0].Filename.Get().Equals(Filename))
			{
				UE_LOGF(LogAutoReimportTests, Error, "Path reported incorrectly (%ls != %ls).", *Changes[0].Filename.Get(), Filename);
			}
			// Check it was the correct type
			else if (Changes[0].Action != DirectoryWatcher::EFileAction::Added)
			{
				UE_LOGF(LogAutoReimportTests, Error, "Incorrect action for added file %ls.", Filename);
			}
			else
			{
				Test->FileCache->CompleteTransaction(MoveTemp(Changes[0]));

				if (!Test->FileCache->FindFileData(Filename))
				{
					UE_LOGF(LogAutoReimportTests, Error, "Add transaction was not applied correctly.");
				}
			}
			return true;
		};

		int32 MaxRetryCounter = LocalMaxRetryCounter;
		const float RetryDelay = LocalRetryDelay;
		RetryLatentCommand(SimpleCreateFunction, RetryDelay, MaxRetryCounter);
	});

	return true;
}

/** Test that modifying an existing file gets reported correctly */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoReimportSimpleModifyTest, "Editor.Auto Reimport.Simple Modify", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAutoReimportSimpleModifyTest::RunTest(const FString& Parameters)
{
	const FString WorkingDir = AutoReimportTests::GetWorkingDir();

	static const TCHAR* SrcFilename1 = TEXT("square.png");
	static const TCHAR* SrcFilename2 = TEXT("red-square.png");
	static const TCHAR* DstFilename = TEXT("square.png");

	// Start watching the directory
	TSharedPtr<FAutoReimportTestPayload> Test = MakeShareable(new FAutoReimportTestPayload(WorkingDir));

	{
		TArray<FSrcDstFilenames> Files;
		Files.Emplace(SrcFilename1, DstFilename);

		if (!CopyTestFiles(*Test, Files))
		{
			return false;
		}
	}
	
	Test->Config.RequireFileHashes(true);
	Test->Config.DetectChangesFor(DirectoryWatcher::FFileCacheConfig::FileHash, true);

	Test->StartWatching();
	Test->WaitForStartup([=]{

		TArray<FSrcDstFilenames> Files;
		Files.Emplace(SrcFilename2, DstFilename);

		if (!CopyTestFiles(*Test, Files))
		{
			return;
		}

		auto SimpleModifyFunction = [=](const bool bLastIteration)->bool
		{
			auto Changes = Test->FileCache->GetOutstandingChanges();
			if (!bLastIteration && Changes.Num() == 0)
			{
				return false;
			}

			if (Changes.Num() != 1)
			{
				UE_LOGF(LogAutoReimportTests, Error, "Incorrect number of changes reported (%d != 1).", Changes.Num());
			}
			// Check that we got a relative path
			else if (!Changes[0].Filename.Get().Equals(DstFilename))
			{
				UE_LOGF(LogAutoReimportTests, Error, "Path reported incorrectly (%ls != %ls).", *Changes[0].Filename.Get(), DstFilename);
			}
			// Check it was the correct type
			else if (Changes[0].Action != DirectoryWatcher::EFileAction::Modified)
			{
				UE_LOGF(LogAutoReimportTests, Error, "Incorrect action for modified file %ls.", DstFilename);
			}
			else
			{
				// Copy the file hash as it will be invalidated by the transaction being completed
				auto FileHash = Changes[0].FileData.FileHash;

				Test->FileCache->CompleteTransaction(MoveTemp(Changes[0]));
				const auto* Data = Test->FileCache->FindFileData(DstFilename);
				if (!Data || Data->FileHash != FileHash)
				{
					UE_LOGF(LogAutoReimportTests, Error, "Modify transaction was not applied correctly.");
				}
			}
			return true;
		};

		int32 MaxRetryCounter = LocalMaxRetryCounter;
		const float RetryDelay = LocalRetryDelay;
		RetryLatentCommand(SimpleModifyFunction, RetryDelay, MaxRetryCounter);
	});

	return true;
}

/** Test that deleting an existing file gets reported correctly */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoReimportSimpleDeleteTest, "Editor.Auto Reimport.Simple Delete", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAutoReimportSimpleDeleteTest::RunTest(const FString& Parameters)
{
	const FString WorkingDir = AutoReimportTests::GetWorkingDir();

	static const TCHAR* Filename = TEXT("square.png");

	TSharedPtr<FAutoReimportTestPayload> Test = MakeShareable(new FAutoReimportTestPayload(WorkingDir));

	TArray<FSrcDstFilenames> Files;
	Files.Emplace(Filename, Filename);

	if (!CopyTestFiles(*Test, Files))
	{
		return false;
	}

	// Start watching the directory
	Test->StartWatching();
	Test->WaitForStartup([=]{

		// Delete the file and check that it got reported as deleted
		const bool bDeleted = IFileManager::Get().Delete(*(Test->Config.Directory / Filename));
		if (!bDeleted)
		{
			UE_LOGF(LogAutoReimportTests, Error, "Failed to delete source file from the test directory.");
			return;
		}

		auto SimpleDeleteFunction = [=](const bool bLastIteration)->bool
		{
			auto Changes = Test->FileCache->GetOutstandingChanges();
			if (!bLastIteration && Changes.Num() == 0)
			{
				return false;
			}

			if (Changes.Num() != 1)
			{
				UE_LOGF(LogAutoReimportTests, Error, "Incorrect number of changes reported (%d != 1).", Changes.Num());
			}
			// Check that we got a relative path
			else if (!Changes[0].Filename.Get().Equals(Filename))
			{
				UE_LOGF(LogAutoReimportTests, Error, "Path reported incorrectly (%ls != %ls).", *Changes[0].Filename.Get(), Filename);
			}
			// Check it was the correct type
			else if (Changes[0].Action != DirectoryWatcher::EFileAction::Removed)
			{
				UE_LOGF(LogAutoReimportTests, Error, "Incorrect action for deleted file %ls.", Filename);
			}
			else
			{
				Test->FileCache->CompleteTransaction(MoveTemp(Changes[0]));

				if (Test->FileCache->FindFileData(Filename))
				{
					UE_LOGF(LogAutoReimportTests, Error, "Remove transaction was not applied correctly.");
				}
			}
			return true;
		};

		int32 MaxRetryCounter = LocalMaxRetryCounter;
		const float RetryDelay = LocalRetryDelay;
		RetryLatentCommand(SimpleDeleteFunction, RetryDelay, MaxRetryCounter);

	});

	return true;
}

/** Test that renaming an existing file gets reported correctly */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoReimportSimpleRenameTest, "Editor.Auto Reimport.Simple Rename", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAutoReimportSimpleRenameTest::RunTest(const FString& Parameters)
{
	const FString WorkingDir = AutoReimportTests::GetWorkingDir();

	static const TCHAR* SrcFilename = TEXT("square.png");
	static const TCHAR* DstFilename = TEXT("square2.png");


	TSharedPtr<FAutoReimportTestPayload> Test = MakeShareable(new FAutoReimportTestPayload(WorkingDir));

	TArray<FSrcDstFilenames> Files;
	Files.Emplace(SrcFilename, SrcFilename);

	if (!CopyTestFiles(*Test, Files))
	{
		return false;
	}

	// Start watching the directory
	Test->StartWatching();
	Test->WaitForStartup([=]{

		// Rename the file
		IFileManager::Get().Move(*(Test->Config.Directory / DstFilename), *(Test->Config.Directory / SrcFilename));

		auto SimpleRenameFunction = [=](const bool bLastIteration)->bool
		{
			auto Changes = Test->FileCache->GetOutstandingChanges();
			if (!bLastIteration && Changes.Num() == 0)
			{
				return false;
			}

			if (Changes.Num() != 1)
			{
				UE_LOGF(LogAutoReimportTests, Error, "Incorrect number of changes reported (%d != 1).", Changes.Num());
			}
			// Check it was the correct type
			else if (Changes[0].Action != DirectoryWatcher::EFileAction::Moved)
			{
				UE_LOGF(LogAutoReimportTests, Error, "Incorrect action for renamed file %ls -> %ls.", SrcFilename, DstFilename);
			}
			else if (!Changes[0].Filename.Get().Equals(DstFilename))
			{
				UE_LOGF(LogAutoReimportTests, Error, "Source path reported incorrectly (%ls != %ls).", *Changes[0].Filename.Get(), DstFilename);
			}
			else if (!Changes[0].MovedFromFilename.Get().Equals(SrcFilename))
			{
				UE_LOGF(LogAutoReimportTests, Error, "Destination path reported incorrectly (%ls != %ls).", *Changes[0].MovedFromFilename.Get(), SrcFilename);
			}
			else
			{
				Test->FileCache->CompleteTransaction(MoveTemp(Changes[0]));

				if (Test->FileCache->FindFileData(SrcFilename) || !Test->FileCache->FindFileData(DstFilename))
				{
					UE_LOGF(LogAutoReimportTests, Error, "Rename transaction was not applied correctly.");
				}
			}
			return true;
		};

		int32 MaxRetryCounter = LocalMaxRetryCounter;
		const float RetryDelay = LocalRetryDelay;
		RetryLatentCommand(SimpleRenameFunction, RetryDelay, MaxRetryCounter);
	});

	return true;
}

/** Test that moving a file outside of the monitored directory gets reported correctly (should be reported as a delete) */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoReimportSimpleMoveExternallyTest, "Editor.Auto Reimport.Move Externally", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAutoReimportSimpleMoveExternallyTest::RunTest(const FString& Parameters)
{
	const FString WorkingDir = AutoReimportTests::GetWorkingDir();

	static const TCHAR* SrcFilename = TEXT("square.png");
	static const TCHAR* DstFilename = TEXT("../square.png");
	
	TSharedPtr<FAutoReimportTestPayload> Test = MakeShareable(new FAutoReimportTestPayload(WorkingDir));

	TArray<FSrcDstFilenames> Files;
	Files.Emplace(SrcFilename, SrcFilename);

	if (!CopyTestFiles(*Test, Files))
	{
		return false;
	}

	// Start watching the directory
	Test->StartWatching();
	Test->WaitForStartup([=]{

		if (!Test->FileCache->FindFileData(SrcFilename))
		{
			UE_LOGF(LogAutoReimportTests, Error, "Could not find file data for initial file %ls.", SrcFilename);
			return;
		}

		// Rename the file
		IFileManager::Get().Move(*(Test->Config.Directory / DstFilename), *(Test->Config.Directory / SrcFilename));

		auto MoveExternallyFunction = [=](const bool bLastIteration)->bool
		{
			auto Changes = Test->FileCache->GetOutstandingChanges();
			if (!bLastIteration && Changes.Num() == 0)
			{
				return false;
			}

			if (Changes.Num() != 1)
			{
				UE_LOGF(LogAutoReimportTests, Error, "Incorrect number of changes reported (%d != 1).", Changes.Num());
			}
			// Check it was the correct type
			else if (Changes[0].Action != DirectoryWatcher::EFileAction::Removed)
			{
				UE_LOGF(LogAutoReimportTests, Error, "Incorrect action for removed file %ls -> %ls.", SrcFilename, DstFilename);
			}
			else if (!Changes[0].Filename.Get().Equals(SrcFilename))
			{
				UE_LOGF(LogAutoReimportTests, Error, "Source path reported incorrectly (%ls != %ls).", *Changes[0].Filename.Get(), SrcFilename);
			}
			else
			{
				Test->FileCache->CompleteTransaction(MoveTemp(Changes[0]));

				if (Test->FileCache->FindFileData(SrcFilename))
				{
					UE_LOGF(LogAutoReimportTests, Error, "Found data for file that should have been removed (%ls).", SrcFilename);
				}
			}
			return true;
		};

		int32 MaxRetryCounter = LocalMaxRetryCounter;
		const float RetryDelay = LocalRetryDelay;
		RetryLatentCommand(MoveExternallyFunction, RetryDelay, MaxRetryCounter);
	});

	return true;
}

/** Test that bDetectChangesSinceLastRun works correctly when true and when false */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoReimportRestartDetectionTest, "Editor.Auto Reimport.Restart Detection", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAutoReimportRestartDetectionTest::RunTest(const FString& Parameters)
{
	const FString WorkingDir = AutoReimportTests::GetWorkingDir();
	static const TCHAR* SrcFilename = TEXT("square.png");
	static const TCHAR* SrcFilename2 = TEXT("red-square.png");
	
	TSharedPtr<FAutoReimportTestPayload> Test = MakeShareable(new FAutoReimportTestPayload(WorkingDir));

	{
		TArray<FSrcDstFilenames> Files;
		Files.Emplace(SrcFilename, SrcFilename);
		if (!CopyTestFiles(*Test, Files))
		{
			return false;
		}
	}

	// Start watching the directory
	Test->StartWatching();
	Test->WaitForStartup([=]{

		if (!Test->FileCache->FindFileData(SrcFilename))
		{
			UE_LOGF(LogAutoReimportTests, Error, "Could not find file data for initial file %ls.", SrcFilename);
			return;
		}

		// Stop watching and write out the cache file
		Test->StopWatching();
		Test->Config.bDetectChangesSinceLastRun = true;

		// Modify the file while the watcher isn't running
		{
			TArray<FSrcDstFilenames> Files;
			Files.Emplace(SrcFilename2, SrcFilename);
			if (!CopyTestFiles(*Test, Files))
			{
				return;
			}
		}

		Test->StartWatching();
		Test->WaitForStartup([=]{

			auto Changes = Test->FileCache->GetOutstandingChanges();

			for (auto& Change : Changes)
			{
				UE_LOGF(LogAutoReimportTests, Log, "Change %d for file %ls.", (uint8)Change.Action, *Change.Filename.Get());
			}

			if (Changes.Num() != 1)
			{
				UE_LOGF(LogAutoReimportTests, Error, "Incorrect number of changes reported (%d != 1).", Changes.Num());
				return;
			}
			// Check it was the correct type
			else if (Changes[0].Action != DirectoryWatcher::EFileAction::Modified)
			{
				UE_LOGF(LogAutoReimportTests, Error, "Incorrect action for modified file %ls.", SrcFilename);
				return;
			}
			else
			{
				Test->FileCache->CompleteTransaction(MoveTemp(Changes[0]));
			}

			// Stop watching and write out the cache file
			Test->StopWatching();
			Test->Config.bDetectChangesSinceLastRun = false;

			// Modify the file while the watcher isn't running
			TArray<FSrcDstFilenames> Files;
			Files.Emplace(SrcFilename, SrcFilename);
			if (!CopyTestFiles(*Test, Files))
			{
				return;
			}

			Test->StartWatching();
			Test->WaitForStartup([=]{

				auto OutstandingChanges = Test->FileCache->GetOutstandingChanges();
				if (OutstandingChanges.Num() != 0)
				{
					UE_LOGF(LogAutoReimportTests, Error, "Shouldn't have reported changes when bDetectChangesSinceLastRun is false (%d change(s) received).", OutstandingChanges.Num());
					return;
				}

			});
		});

	});

	return true;
}

/** Test that making multiple changes to the same file gets picked up correctly */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoReimportMultipleChangesTest, "Editor.Auto Reimport.Multiple Changes", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAutoReimportMultipleChangesTest::RunTest(const FString& Parameters)
{
	const FString WorkingDir = AutoReimportTests::GetWorkingDir();
	static const TCHAR* Filename1 = TEXT("square.png");
	static const TCHAR* Filename2 = TEXT("red-square.png");
	
	TSharedPtr<FAutoReimportTestPayload> Test = MakeShareable(new FAutoReimportTestPayload(WorkingDir));

	// Start watching the directory
	Test->StartWatching();
	Test->WaitForStartup([=]{

		// Add Filename1 and Filename2
		TArray<FSrcDstFilenames> Files;
		Files.Emplace(Filename1, Filename1);
		Files.Emplace(Filename2, Filename2);
		if (!CopyTestFiles(*Test, Files))
		{
			return;
		}

		// Delete Filename1
		IFileManager::Get().Delete(*(Test->Config.Directory / Filename1));

		// Modify Filename2
		Files.Reset();
		Files.Emplace(Filename1, Filename2);
		if (!CopyTestFiles(*Test, Files))
		{
			return;
		}

		auto MultipleChangesFunction = [=](const bool bLastIteration)->bool
		{
			// The net change should just be a single added file
			auto Changes = Test->FileCache->GetOutstandingChanges();
			if (!bLastIteration && Changes.Num() == 0)
			{
				return false;
			}

			if (Changes.Num() != 1)
			{
				UE_LOGF(LogAutoReimportTests, Error, "Incorrect number of changes reported (%d != 1).", Changes.Num());
			}
			// Check it was the correct type
			else if (Changes[0].Action != DirectoryWatcher::EFileAction::Added)
			{
				UE_LOGF(LogAutoReimportTests, Error, "Incorrect action for added file %ls.", Filename2);
			}
			return true;
		};

		int32 MaxRetryCounter = LocalMaxRetryCounter;
		const float RetryDelay = LocalRetryDelay;
		RetryLatentCommand(MultipleChangesFunction, RetryDelay, MaxRetryCounter);
	});

	return true;
}

/** Test that starting up a cache file with a different set of applicable extensions correctly ignores/updates the extensions, whilst reporting changes only for applicable extensions */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoReimportChangeExtensionsTest, "Editor.Auto Reimport.Change Extensions", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAutoReimportChangeExtensionsTest::RunTest(const FString& Parameters)
{
	const FString WorkingDir = AutoReimportTests::GetWorkingDir();
	static const TCHAR* SrcFilename1 = TEXT("square.png");
	static const TCHAR* SrcFilename2 = TEXT("red-square.png");
	static const TCHAR* SrcFilename3 = TEXT("empty.txt");
	
	TSharedPtr<FAutoReimportTestPayload> Test = MakeShareable(new FAutoReimportTestPayload(WorkingDir));
	Test->Config.Rules.SetApplicableExtensions(TEXT("txt;"));
	Test->Config.bDetectChangesSinceLastRun = true;

	// Start watching the directory
	Test->StartWatching();
	Test->WaitForStartup([=]{

		// Add a txt file and a png - we should only be told about the txt
		{
			TArray<FSrcDstFilenames> Files;
			Files.Emplace(SrcFilename1, SrcFilename1);
			Files.Emplace(SrcFilename3, SrcFilename3);
			if (!CopyTestFiles(*Test, Files))
			{
				return;
			}
		}

		auto ChangeExtensionsFunction = [=](const bool bLastIteration)->bool
		{
			// The net change should just be a single added file
			auto Changes = Test->FileCache->GetOutstandingChanges();
			if (!bLastIteration && Changes.Num() == 0)
			{
				return false;
			}

			if (Changes.Num() != 1)
			{
				UE_LOGF(LogAutoReimportTests, Error, "Incorrect number of changes reported (%d != 1).", Changes.Num());
				return true;
			}
			else if (Changes[0].Action != DirectoryWatcher::EFileAction::Added)
			{
				UE_LOGF(LogAutoReimportTests, Error, "Incorrect action for added file %ls.", SrcFilename3);
				return true;
			}
			else if (!Changes[0].Filename.Get().Equals(SrcFilename3))
			{
				UE_LOGF(LogAutoReimportTests, Error, "Added file path reported incorrectly (%ls != %ls).", *Changes[0].Filename.Get(), SrcFilename3);
				return true;
			}
			Test->FileCache->CompleteTransaction(MoveTemp(Changes[0]));

			Test->StopWatching();
			// Change the extensions we're watching, and add another png
			Test->Config.Rules = DirectoryWatcher::FMatchRules();
			Test->Config.Rules.SetApplicableExtensions(TEXT("png;"));

			Test->StartWatching();
			Test->WaitForStartup([=]{

				{
					auto OutstandingChanges = Test->FileCache->GetOutstandingChanges();
					if (OutstandingChanges.Num() != 0)
					{
						UE_LOGF(LogAutoReimportTests, Error, "Incorrect number of changes reported (%d != 0).", OutstandingChanges.Num());
						return;
					}
				}

				// Add another png - we should only be notified about this one
				TArray<FSrcDstFilenames> Files;
				Files.Emplace(SrcFilename2, SrcFilename2);
				if (!CopyTestFiles(*Test, Files))
				{
					return;
				}

				auto ChangeExtensionsInternalFunction = [=](const bool bLastIteration)->bool
				{
					// The net change should just be a single added file
					auto OutstandingChanges = Test->FileCache->GetOutstandingChanges();
					if (!bLastIteration && OutstandingChanges.Num() == 0)
					{
						return false;
					}

					if (OutstandingChanges.Num() != 1)
					{
						UE_LOGF(LogAutoReimportTests, Error, "Incorrect number of changes reported (%d != 1).", OutstandingChanges.Num());
					}
					return true;
				};

				int32 MaxRetryCounter2 = LocalMaxRetryCounter;
				const float RetryDelay2 = LocalRetryDelay;
				RetryLatentCommand(ChangeExtensionsInternalFunction, RetryDelay2, MaxRetryCounter2);

			});
			return true;
		};

		int32 MaxRetryCounter = LocalMaxRetryCounter;
		const float RetryDelay = LocalRetryDelay;
		RetryLatentCommand(ChangeExtensionsFunction, RetryDelay, MaxRetryCounter);
	});

	return true;
}

/** Test that starting up a cache file with a different set of applicable extensions correctly ignores/updates the extensions, whilst reporting changes only for applicable extensions */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAutoReimportWildcardFiltersTest, "Editor.Auto Reimport.Wildcard Filters", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FAutoReimportWildcardFiltersTest::RunTest(const FString& Parameters)
{
	const FString WorkingDir = AutoReimportTests::GetWorkingDir();
	static const TCHAR* SrcFilename1 = TEXT("square.png");
	static const TCHAR* SrcFilename2 = TEXT("red-square.png");
	static const TCHAR* SrcFilename3 = TEXT("empty.txt");

	static const TCHAR* DstFilename1 = TEXT("sub-folder/square.png");
	static const TCHAR* DstFilename2 = TEXT("red-square.png");
	static const TCHAR* DstFilename3 = TEXT("sub-folder/empty.txt");
	
	TSharedPtr<FAutoReimportTestPayload> Test = MakeShareable(new FAutoReimportTestPayload(WorkingDir));
	Test->Config.Rules.SetApplicableExtensions(TEXT("txt;png;"));
	Test->Config.Rules.AddWildcardRule(TEXT("sub-folder/*"), true);
	Test->Config.Rules.AddWildcardRule(TEXT("sub-folder/*.png"), false);
	Test->Config.bDetectChangesSinceLastRun = true;
	Test->Config.RequireFileHashes(true);
	Test->Config.DetectChangesFor(DirectoryWatcher::FFileCacheConfig::FileHash, true);

	IFileManager::Get().MakeDirectory(*( AutoReimportTests::GetWorkingDir() / TEXT("Content") / TEXT("sub-folder")), false);
		
	{
		TArray<FSrcDstFilenames> Files;
		Files.Emplace(SrcFilename1, DstFilename1);
		Files.Emplace(SrcFilename2, DstFilename2);
		Files.Emplace(SrcFilename3, DstFilename3);
		if (!CopyTestFiles(*Test, Files))
		{
			return false;
		}
	}

	// Start watching the directory
	Test->StartWatching();
	Test->WaitForStartup([=]{

		if (!Test->FileCache->FindFileData(DstFilename3))
		{
			UE_LOGF(LogAutoReimportTests, Error, "Couldn't find data for %ls.", DstFilename3);
		}
		else if (Test->FileCache->FindFileData(DstFilename2))
		{
			UE_LOGF(LogAutoReimportTests, Error, "Erroneously found data for %ls.", DstFilename2);
		}
		else if (Test->FileCache->FindFileData(DstFilename1))
		{
			UE_LOGF(LogAutoReimportTests, Error, "Erroneously found data for %ls.", DstFilename1);
		}
		else
		{
			// Check that we get the same behavior for live changes
			TArray<FSrcDstFilenames> Files;
			Files.Emplace(SrcFilename2, DstFilename1);
			Files.Emplace(SrcFilename1, DstFilename2);
			Files.Emplace(SrcFilename2, DstFilename3);
			if (!CopyTestFiles(*Test, Files))
			{
				return;
			}

			auto WildcardFiltersFunction = [=](const bool bLastIteration)->bool
			{
				// The net change should just be a single added file
				auto Changes = Test->FileCache->GetOutstandingChanges();
				if (!bLastIteration && Changes.Num() == 0)
				{
					return false;
				}

				if (Changes.Num() != 1)
				{
					UE_LOGF(LogAutoReimportTests, Error, "Incorrect number of changes reported (%d != 1).", Changes.Num());
				}
				else if (!Changes[0].Filename.Get().Equals(DstFilename3))
				{
					UE_LOGF(LogAutoReimportTests, Error, "Modified file path reported incorrectly (%ls != %ls).", *Changes[0].Filename.Get(), DstFilename3);
				}
				return true;
			};

			int32 MaxRetryCounter = LocalMaxRetryCounter;
			const float RetryDelay = LocalRetryDelay;
			RetryLatentCommand(WildcardFiltersFunction, RetryDelay, MaxRetryCounter);
		}

	});

	return true;
}
