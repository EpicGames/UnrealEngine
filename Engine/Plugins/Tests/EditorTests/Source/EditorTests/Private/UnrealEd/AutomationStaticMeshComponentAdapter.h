// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StaticMeshComponentAdapter.h"
#include "UObject/Package.h"

/** Adapter which supplies the transient package as outer to store the data in, used for automation testing */
class FAutomationStaticMeshComponentAdapter : public FStaticMeshComponentAdapter
{
public:
	FAutomationStaticMeshComponentAdapter(UStaticMeshComponent* InStaticMeshComponent) : FStaticMeshComponentAdapter(InStaticMeshComponent) {}
	virtual UPackage* GetOuter() const override
	{
		return GetTransientPackage();
	}
};