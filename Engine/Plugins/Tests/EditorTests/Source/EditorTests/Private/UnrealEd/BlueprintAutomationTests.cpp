// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/AutomationTest.h"
#include "Modules/ModuleManager.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "UObject/GarbageCollection.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/MetaData.h"
#include "UObject/UnrealType.h"
#include "Serialization/ObjectWriter.h"
#include "Serialization/ObjectReader.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "Misc/PackageName.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Engine/Blueprint.h"
#include "GameFramework/Actor.h"
#include "AssetRegistry/AssetData.h"
#include "Animation/AnimBlueprint.h"
#include "GameFramework/SaveGame.h"
#include "EngineUtils.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "FileHelpers.h"

#include "ObjectTools.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"


#include "Kismet2/KismetEditorUtilities.h"
#include "EdGraphSchema_K2.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "DiffResults.h"
#include "GraphDiffControl.h"
#include "Subsystems/AssetEditorSubsystem.h"

DEFINE_LOG_CATEGORY_STATIC(LogBlueprintAutomationTests, Log, All);

IMPLEMENT_COMPLEX_AUTOMATION_TEST(FCompileBlueprintsTest, "Project.Blueprints.Compile Blueprints", EAutomationTestFlags::EditorContext | EAutomationTestFlags::StressFilter)
IMPLEMENT_COMPLEX_AUTOMATION_TEST(FCompileAnimBlueprintsTest, "Project.Blueprints.Compile Anims", EAutomationTestFlags::EditorContext | EAutomationTestFlags::StressFilter)

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCompileMissingNodeBlueprint, "Project.Functional Tests.Tests.Blueprints.Compile Missing Node", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

class FBlueprintAutomationTestUtilities
{
	/** An incrementing number that can be used to tack on to save files, etc. (for avoiding naming conflicts)*/
	static uint32 QueuedTempId;

	/** List of packages touched by automation tests that can no longer be saved */
	static TArray<FName> DontSavePackagesList;

	/** Callback to check if package is ok to save. */
	static bool IsPackageOKToSave(UPackage* InPackage, const FString& InFilename, FOutputDevice* Error)
	{
		return !DontSavePackagesList.Contains(InPackage->GetFName());
	}

	/**
	 * Gets a unique int (this run) for automation purposes (to avoid temp save 
	 * file collisions, etc.)
	 * 
	 * @return A id for unique identification that can be used to tack on to save files, etc. (for avoiding naming conflicts)
	 */
	static uint32 GenTempUid()
	{
		return QueuedTempId++;
	}

public:

	typedef TMap< FString, FString >	FPropertiesMap;

	/** 
	 * Helper struct to ensure that a package is not inadvertently left in
	 * a dirty state by automation tests
	 */
	struct FPackageCleaner
	{
		FPackageCleaner(UPackage* InPackage) : Package(InPackage)
		{
			bIsDirty = Package ? Package->IsDirty() : false;
		}

		~FPackageCleaner()
		{
			// reset the dirty flag
			if (Package) 
			{
				Package->SetDirtyFlag(bIsDirty);
			}
		}

	private:
		bool bIsDirty;
		UPackage* Package;
	};

	/**
	 * Loads the map specified by an automation test
	 * 
	 * @param MapName - Map to load
	 */
	static void LoadMap(const FString& MapName)
	{
		const bool bLoadAsTemplate = false;
		const bool bShowProgress = false;

		FEditorFileUtils::LoadMap(MapName, bLoadAsTemplate, bShowProgress);
	}

	/** 
	 * Filter used to test to see if a FProperty is candidate for comparison.
	 * @param Property	The property to test
	 *
	 * @return True if FProperty should be compared, false otherwise
	 */
	static bool ShouldCompareProperty (const FProperty* Property)
	{
		// Ignore components & transient properties
		const bool bIsTransient = !!( Property->PropertyFlags & CPF_Transient );
		const bool bIsComponent = !!( Property->PropertyFlags & (CPF_InstancedReference | CPF_ContainsInstancedReference) );
		const bool bShouldCompare = !(bIsTransient || bIsComponent);

		return (bShouldCompare && Property->HasAnyPropertyFlags(CPF_BlueprintVisible));
	}
	
	/** 
	 * Get a given UObject's properties in simple key/value string map
	 *
	 * @param Obj			The UObject to extract properties for 
	 * @param ObjProperties	[OUT] Property map to be filled
	 */
	static void GetObjProperties (UObject* Obj, FPropertiesMap& ObjProperties)
	{
		for (TFieldIterator<FProperty> PropIt(Obj->GetClass(), EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
		{
			FProperty* Prop = *PropIt;

			if ( ShouldCompareProperty(Prop) )
			{
				for (int32 Index = 0; Index < Prop->ArrayDim; Index++)
				{
					FString PropName = (Prop->ArrayDim > 1) ? FString::Printf(TEXT("%s[%d]"), *Prop->GetName(), Index) : *Prop->GetName();
					FString PropText;
					Prop->ExportText_InContainer(Index, PropText, Obj, Obj, Obj, PPF_SimpleObjectText);
					ObjProperties.Add(PropName, PropText);
				}
			}
		}
	}

	/** 
	 * Compare two object property maps
	 * @param OrigName		The name of the original object being compared against
	 * @param OrigMap		The property map for the object
	 * @param CmpName		The name of the object to compare
	 * @param CmpMap		The property map for the object to compare
	 */
	static bool ComparePropertyMaps(FName OrigName, TMap<FString, FString>& OrigMap, FName CmpName, FPropertiesMap& CmpMap, FCompilerResultsLog& Results)
	{
		if (OrigMap.Num() != CmpMap.Num())
		{
			Results.Error( *FString::Printf(TEXT("Objects have a different number of properties (%d vs %d)"), OrigMap.Num(), CmpMap.Num()) );
			return false;
		}

		bool bMatch = true;
		for (auto PropIt = OrigMap.CreateIterator(); PropIt; ++PropIt)
		{
			FString Key = PropIt.Key();
			FString Val = PropIt.Value();

			const FString* CmpValue = CmpMap.Find(Key);

			// Value is missing
			if (CmpValue == NULL)
			{
				bMatch = false;
				Results.Error( *FString::Printf(TEXT("Property is missing in object being compared: (%s %s)"), *Key, *Val) );
				break;
			}
			else if (Val != *CmpValue)
			{
				// string out object names and retest
				FString TmpCmp(*CmpValue);
				TmpCmp.ReplaceInline(*CmpName.ToString(), TEXT(""));
				FString TmpVal(Val);
				TmpVal.ReplaceInline(*OrigName.ToString(), TEXT(""));

				if (TmpCmp != TmpVal)
				{
					bMatch = false;
					Results.Error( *FString::Printf(TEXT("Object properties do not match: %s (%s vs %s)"), *Key, *Val, *(*CmpValue)) );
					break;
				}
			}
		}
		return bMatch;
	}

	/** 
	 * Compares the properties of two UObject instances

	 * @param OriginalObj		The first UObject to compare
	 * @param CompareObj		The second UObject to compare
	 * @param Results			The results log to record errors or warnings
	 *
	 * @return True of the blueprints are the same, false otherwise (see the Results log for more details)
	 */
	static bool CompareObjects(UObject* OriginalObj, UObject* CompareObj, FCompilerResultsLog& Results)
	{
		bool bMatch = false;

		// ensure we have something sensible to compare
		if (OriginalObj == NULL)
		{
			Results.Error( TEXT("Original object is null") );
			return false;
		}
		else if (CompareObj == NULL)
		{	
			Results.Error( TEXT("Compare object is null") );
			return false;
		}
		else if (OriginalObj == CompareObj)
		{
			Results.Error( TEXT("Objects to compare are the same") );
			return false;
		}

		TMap<FString, FString> ObjProperties;
		GetObjProperties(OriginalObj, ObjProperties);

		TMap<FString, FString> CmpProperties;
		GetObjProperties(CompareObj, CmpProperties);

		return ComparePropertyMaps(OriginalObj->GetFName(), ObjProperties, CompareObj->GetFName(), CmpProperties, Results);
	}

	/** 
	 * Runs over all the assets looking for ones that can be used by this test
	 *
	 * @param Class					The UClass of assets to load for the test
	 * @param OutBeautifiedNames	[Filled by method] The beautified filenames of the assets to use in the test
	 * @param OutTestCommands		[Filled by method] The asset name in a form suitable for passing as a param to the test
	 * @param bIgnoreLoaded			If true, ignore any already loaded assets
	 */
	static void CollectTestsByClass(UClass * Class, TArray< FString >& OutBeautifiedNames, TArray< FString >& OutTestCommands, bool bIgnoreLoaded) 
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		TArray<FAssetData> ObjectList;
		AssetRegistryModule.Get().GetAssetsByClass(Class->GetClassPathName(), ObjectList);

		for (auto ObjIter=ObjectList.CreateConstIterator(); ObjIter; ++ObjIter)
		{
			const FAssetData& Asset = *ObjIter;
			FString Filename = Asset.GetObjectPathString();
			//convert to full paths
			Filename = FPackageName::LongPackageNameToFilename(Filename);
			if (FAutomationTestFramework::Get().ShouldTestContent(Filename))
			{
				// optionally discount already loaded assets
				if (!bIgnoreLoaded || !Asset.IsAssetLoaded())
				{
					FString BeautifiedFilename = Asset.AssetName.ToString();
					OutBeautifiedNames.Add(BeautifiedFilename);
					OutTestCommands.Add(Asset.GetObjectPathString());
				}
			}
		}
	}

	/**
	 * Adds a package to a list of packages that can no longer be saved.
	 *
	 * @param Package Package to prevent from being saved to disk.
	 */
	static void DontSavePackage(UPackage* Package)
	{
		if (DontSavePackagesList.Num() == 0)
		{
			FCoreUObjectDelegates::IsPackageOKToSaveDelegate.BindStatic(&IsPackageOKToSave);
		}
		DontSavePackagesList.AddUnique(Package->GetFName());
	}

	/**
	 * A helper method that will reset a package for reload, and flag it as 
	 * unsavable (meant to be used after you've messed with a package for testing 
	 * purposes, leaving it in a questionable state).
	 * 
	 * @param  Package	The package you wish to invalidate.
	 */
	static void InvalidatePackage(UPackage* const Package)
	{
		// reset the blueprint's original package/linker so that we can get by
		// any early returns (in the load code), and reload its exports as if new 
		ResetLoaders(Package);
		Package->ClearFlags(RF_WasLoaded);	
		Package->bHasBeenFullyLoaded = false;

		Package->GetMetaData().RemoveMetaDataOutsidePackage(Package);
		// we've mucked around with the package manually, we should probably prevent 
		// people from saving it in this state (this means you won't be able to save 
		// the blueprints that these tests were run on until you relaunch the editor)
		DontSavePackage(Package);
	}

	/**
	 * Helper method to close a specified blueprint (if it is open in the blueprint-editor).
	 * 
	 * @param  BlueprintObj		The blueprint you want the editor closed for.
	 */
	static void CloseBlueprint(UBlueprint* const BlueprintObj)
	{
		IAssetEditorInstance* EditorInst = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(BlueprintObj, /*bool bFocusIfOpen =*/false);
		if (EditorInst != nullptr)
		{
			UE_LOGF(LogBlueprintAutomationTests, Log, "Closing '%ls' so we don't invalidate the open version when unloading it.", *BlueprintObj->GetName());
			EditorInst->CloseWindow(EAssetEditorCloseReason::AssetUnloadingOrInvalid);
		}
	}

	/** 
	 * Helper method to unload loaded blueprints. Use with caution.
	 *
	 * @param BlueprintObj	The blueprint to unload
	 * @param bForceFlush   If true, will force garbage collection on everything but native objects (defaults to false)
	 */
	static void UnloadBlueprint(UBlueprint* const BlueprintObj, bool bForceFlush = false)
	{
		// have to grab the blueprint's package before we move it to the transient package
		UPackage* const OldPackage = BlueprintObj->GetOutermost();

		UPackage* const TransientPackage = GetTransientPackage();
		if (OldPackage == TransientPackage)
		{
			UE_LOGF(LogBlueprintAutomationTests, Log, "No need to unload '%ls' from the transient package.", *BlueprintObj->GetName());
		}
		else if (OldPackage->IsRooted() || BlueprintObj->IsRooted())
		{
			UE_LOGF(LogBlueprintAutomationTests, Error, "Cannot unload '%ls' when its root is set (it will not be garbage collected, leaving it in an erroneous state).", *OldPackage->GetName());
		}
		else if (OldPackage->IsDirty())
		{
			UE_LOGF(LogBlueprintAutomationTests, Error, "Cannot unload '%ls' when it has unsaved changes (save the asset and then try again).", *OldPackage->GetName());
		}
		else 
		{
			// prevent users from modifying an open blueprint, after it has been unloaded
			CloseBlueprint(BlueprintObj);

			UPackage* NewPackage = TransientPackage;
			// move the blueprint to the transient package (to be picked up by garbage collection later)
			FName UnloadedName = MakeUniqueObjectName(NewPackage, UBlueprint::StaticClass(), BlueprintObj->GetFName());
			BlueprintObj->Rename(*UnloadedName.ToString(), NewPackage, REN_DontCreateRedirectors|REN_DoNotDirty);

			// Rename() will mark the OldPackage dirty (since it is removing the
			// blueprint from it), we don't want this to affect the dirty flag 
			// (for if/when we load it again)
			OldPackage->SetDirtyFlag(/*bIsDirty =*/false);

			// make sure the blueprint is properly trashed so we can rerun tests on it
			BlueprintObj->SetFlags(RF_Transient);
			BlueprintObj->ClearFlags(RF_Standalone | RF_Transactional);
			BlueprintObj->RemoveFromRoot();
			BlueprintObj->MarkAsGarbage();

			InvalidatePackage(OldPackage);
		}

		// because we just emptied out an existing package, we may want to clean 
		// up garbage so an attempted load doesn't stick us with an invalid asset
		if (bForceFlush)
		{
#if WITH_EDITOR
			// clear undo history to ensure that the transaction buffer isn't 
			// holding onto any references to the blueprints we want unloaded
			GEditor->Trans->Reset(NSLOCTEXT("BpAutomation", "BpAutomationTest", "Blueprint Automation Test"));
#endif // #if WITH_EDITOR
			CollectGarbage(RF_NoFlags);
		}
	}

	/**
	 * A utility function to help separate a package name and asset name out 
	 * from a full asset object path.
	 * 
	 * @param  AssetObjPathIn	The asset object path you want split.
	 * @param  PackagePathOut	The first half of the in string (the package portion).
	 * @param  AssetNameOut		The second half of the in string (the asset name portion).
	 */
	static void SplitPackagePathAndAssetName(FString const& AssetObjPathIn, FString& PackagePathOut, FString& AssetNameOut)
	{
		AssetObjPathIn.Split(TEXT("."), &PackagePathOut, &AssetNameOut);
	}

	/**
	 * A utility function for looking up a package from an asset's full path (a 
	 * long package path).
	 * 
	 * @param  AssetPath	The object path for a package that you want to look up.
	 * @return A package containing the specified asset (NULL if the asset doesn't exist, or it isn't loaded).
	 */
	static UPackage* FindPackageForAsset(FString const& AssetPath)
	{
		FString PackagePath, AssetName;
		SplitPackagePathAndAssetName(AssetPath, PackagePath, AssetName);

		return FindPackage(NULL, *PackagePath);
	}

	/**
	 * Helper method for checking to see if a blueprint is currently loaded.
	 * 
	 * @param  AssetPath	A path detailing the asset in question (in the form of <PackageName>.<AssetName>)
	 * @return True if a blueprint can be found with a matching package/name, false if no corresponding blueprint was found.
	 */
	static bool IsBlueprintLoaded(FString const& AssetPath, UBlueprint** BlueprintOut = nullptr)
	{
		bool bIsLoaded = false;

		if (UPackage* ExistingPackage = FindPackageForAsset(AssetPath))
		{
			FString PackagePath, AssetName;
			SplitPackagePathAndAssetName(AssetPath, PackagePath, AssetName);

			if (UBlueprint* ExistingBP = Cast<UBlueprint>(StaticFindObject(UBlueprint::StaticClass(), ExistingPackage, *AssetName)))
			{
				bIsLoaded = true;
				if (BlueprintOut != nullptr)
				{
					*BlueprintOut = ExistingBP;
				}
			}
		}

		return bIsLoaded;
	}

	/** */
	static bool GetExternalReferences(UObject* Obj, TArray<FReferencerInformation>& ExternalRefsOut)
	{
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

		bool bHasReferences = false;

		FReferencerInformationList Refs;
		if (IsReferenced(Obj, RF_Public, EInternalObjectFlags::None, true, &Refs))
		{
			ExternalRefsOut = Refs.ExternalReferences;
			bHasReferences = true;
		}

		return bHasReferences;
	}

	/**
	 * Helper method for determining if the specified asset has pending changes.
	 * 
	 * @param  AssetPath	The object path to an asset you want checked.
	 * @return True if the package is unsaved, false if it is up to date.
	 */
	static bool IsAssetUnsaved(FString const& AssetPath)
	{
		bool bIsUnsaved = false;
		if (UPackage* ExistingPackage = FindPackageForAsset(AssetPath))
		{
			bIsUnsaved = ExistingPackage->IsDirty();
		}
		return bIsUnsaved;
	}

	/**
	 * Simulates the user pressing the blueprint's compile button (will load the
	 * blueprint first if it isn't already).
	 * 
	 * @param  BlueprintAssetPath	The asset object path that you wish to compile.
	 * @return False if we failed to load the blueprint, true otherwise
	 */
	static bool CompileBlueprint(const FString& BlueprintAssetPath)
	{
		UBlueprint* BlueprintObj = Cast<UBlueprint>(StaticLoadObject(UBlueprint::StaticClass(), NULL, *BlueprintAssetPath));
		if (!BlueprintObj || !BlueprintObj->ParentClass)
		{
			UE_LOGF(LogBlueprintAutomationTests, Error, "Failed to compile invalid blueprint, or blueprint parent no longer exists.");
			return false;
		}

		UPackage* const BlueprintPackage = BlueprintObj->GetOutermost();
		// compiling the blueprint will inherently dirty the package, but if there 
		// weren't any changes to save before, there shouldn't be after
		bool const bStartedWithUnsavedChanges = (BlueprintPackage != nullptr) ? BlueprintPackage->IsDirty() : true;

		FKismetEditorUtilities::CompileBlueprint(BlueprintObj, EBlueprintCompileOptions::SkipGarbageCollection);

		if (BlueprintPackage != nullptr)
		{
			BlueprintPackage->SetDirtyFlag(bStartedWithUnsavedChanges);
		}

		return true;
	}

	/**
	 * Takes two blueprints and compares them (as if we were running the in-editor 
	 * diff tool). Any discrepancies between the two graphs will be listed in the DiffsOut array.
	 * 
	 * @param  LhsBlueprint	The baseline blueprint you wish to compare against.
	 * @param  RhsBlueprint	The blueprint you wish to look for changes in.
	 * @param  DiffsOut		An output list that will contain any graph internal differences that were found.
	 * @return True if the two blueprints differ, false if they are identical.
	 */
	static bool DiffBlueprints(UBlueprint* const LhsBlueprint, UBlueprint* const RhsBlueprint, TArray<FDiffSingleResult>& DiffsOut)
	{
		TArray<UEdGraph*> LhsGraphs;
		LhsBlueprint->GetAllGraphs(LhsGraphs);
		TArray<UEdGraph*> RhsGraphs;
		RhsBlueprint->GetAllGraphs(RhsGraphs);

		bool bDiffsFound = false;
		// walk the graphs in the rhs blueprint (because, conceptually, it is the more up to date one)
		for (auto RhsGraphIt(RhsGraphs.CreateIterator()); RhsGraphIt; ++RhsGraphIt)
		{
			UEdGraph* RhsGraph = *RhsGraphIt;
			UEdGraph* LhsGraph = NULL;

			// search for the corresponding graph in the lhs blueprint
			for (auto LhsGraphIt(LhsGraphs.CreateIterator()); LhsGraphIt; ++LhsGraphIt)
			{
				// can't trust the guid until we've done a resave on every asset
				//if ((*LhsGraphIt)->GraphGuid == RhsGraph->GraphGuid)
				
				// name compares is probably sufficient, but just so we don't always do a string compare
				if (((*LhsGraphIt)->GetClass() == RhsGraph->GetClass()) &&
					((*LhsGraphIt)->GetName() == RhsGraph->GetName()))
				{
					LhsGraph = *LhsGraphIt;
					break;
				}
			}

			// if a matching graph wasn't found in the lhs blueprint, then that is a BIG inconsistency
			if (LhsGraph == NULL)
			{
				bDiffsFound = true;
				continue;
			}

			bDiffsFound |= FGraphDiffControl::DiffGraphs(LhsGraph, RhsGraph, DiffsOut);
		}

		return bDiffsFound;
	}

	/**
	 * Gathers a list of asset files corresponding to a config array (an array 
	 * of package paths).
	 * 
	 * @param  ConfigKey	A key to the config array you want to look up.
	 * @param  AssetsOut	An output array that will be filled with the desired asset data.
	 * @param  ClassType	If specified, will further filter the asset look up by class.
	 */
	static void GetAssetListingFromConfig(FString const& ConfigKey, TArray<FAssetData>& AssetsOut, UClass const* const ClassType = NULL)
	{
		check(GConfig != NULL);

		FARFilter AssetFilter;
		AssetFilter.bRecursivePaths = true;
		if (ClassType != NULL)
		{
			AssetFilter.ClassPaths.Add(ClassType->GetClassPathName());
		}
		
		TArray<FString> AssetPaths;
		GConfig->GetArray(TEXT("AutomationTesting.Blueprint"), *ConfigKey, AssetPaths, GEngineIni);
		for (FString const& AssetPath : AssetPaths)
		{
			AssetFilter.PackagePaths.Add(*AssetPath);
		}
		
		if (AssetFilter.PackagePaths.Num() > 0)
		{
			IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
			AssetRegistry.GetAssets(AssetFilter, AssetsOut);
		}
	}

	/**
	 * A utility function for spawning an empty temporary package, meant for test purposes.
	 * 
	 * @param  Name		A suggested string name to get the package (NOTE: it will further be decorated with things like "Temp", etc.) 
	 * @return A newly created package for throwaway use.
	 */
	static UPackage* CreateTempPackage(FString Name)
	{
		FString TempPackageName = FString::Printf(TEXT("/Temp/BpAutomation-%u-%s"), GenTempUid(), *Name);
		return CreatePackage(*TempPackageName);
	}

	/**
	 * A helper that will take a blueprint and copy it into a new, temporary 
	 * package (intended for throwaway purposes).
	 * 
	 * @param  BlueprintToClone		The blueprint you wish to duplicate.
	 * @return A new temporary blueprint copy of what was passed in.
	 */
	static UBlueprint* DuplicateBlueprint(UBlueprint const* const BlueprintToClone)
	{
		UPackage* TempPackage = CreateTempPackage(BlueprintToClone->GetName());

		const FName TempBlueprintName = MakeUniqueObjectName(TempPackage, UBlueprint::StaticClass(), BlueprintToClone->GetFName());
		return Cast<UBlueprint>(StaticDuplicateObject(BlueprintToClone, TempPackage, TempBlueprintName));
	}

	/**
	 * A getter function for coordinating between multiple tests, a place for 
	 * temporary test files to be saved.
	 * 
	 * @return A relative path to a directory meant for temp automation files.
	 */
	static FString GetTempDir()
	{
		return FPaths::ProjectSavedDir() + TEXT("Automation/");
	}

	/**
	 * Will save a blueprint package under a temp file and report on weather it succeeded or not.
	 * 
	 * @param  BlueprintObj		The blueprint you wish to save.
	 * @return True if the save was successful, false if it failed.
	 */
	static bool TestSaveBlueprint(UBlueprint* const BlueprintObj)
	{
		FString TempDir = GetTempDir();
		IFileManager::Get().MakeDirectory(*TempDir);

		FString SavePath = FString::Printf(TEXT("%sTemp-%u-%s"), *TempDir, GenTempUid(), *FPaths::GetCleanFilename(BlueprintObj->GetName()));

		UPackage* const AssetPackage = BlueprintObj->GetOutermost();
		FSavePackageArgs SaveArgs;
		SaveArgs.TopLevelFlags = RF_Standalone;
		SaveArgs.Error = GWarn;
		return UPackage::SavePackage(AssetPackage, NULL, *SavePath, SaveArgs);
	}

	static void ResolveCircularDependencyDiffs(UBlueprint const* const BlueprintIn, TArray<FDiffSingleResult>& DiffsInOut)
	{
		UEdGraphSchema_K2 const* K2Schema = GetDefault<UEdGraphSchema_K2>();

		typedef TArray<FDiffSingleResult>::TIterator TDiffIt;
		TMap<UEdGraphPin*, TDiffIt> PinLinkDiffsForRepair;

		for (auto DiffIt(DiffsInOut.CreateIterator()); DiffIt; ++DiffIt)
		{
			// as far as we know, pin link diffs are the only ones that would be
			// affected by circular references pointing to an unloaded class 
			// 
			// NOTE: we only handle PIN_LINKEDTO_NUM_INC over PIN_LINKEDTO_NUM_DEC,
			//       this assumes that the diff was performed in a specific 
			//       order (the reloaded blueprint first).
			if (DiffIt->Diff != EDiffType::PIN_LINKEDTO_NUM_INC)
			{
				continue;
			}

			check(DiffIt->Pin1 != nullptr);
			check(DiffIt->Pin2 != nullptr);
			UEdGraphPin* MalformedPin = DiffIt->Pin1;
			
			FEdGraphPinType const& PinType = MalformedPin->PinType;
			// only object pins would reference the unloaded blueprint
			if (!PinType.PinSubCategoryObject.IsValid() || ((PinType.PinCategory != UEdGraphSchema_K2::PC_Object) && 
				(PinType.PinCategory != UEdGraphSchema_K2::PSC_Self) && (PinType.PinCategory != UEdGraphSchema_K2::PC_Interface)))
			{
				continue;
			}

			UStruct const* PinObjType = Cast<UStruct>(PinType.PinSubCategoryObject.Get());
			// only pins that match the blueprint class would have been affected 
			// by the unload (assumes an FArchiveReplaceObjectRef() has since been 
			// ran to fix-up any references to the unloaded class... meaning the 
			// malformed pins now have the proper reference)
			if (!PinObjType->IsChildOf(BlueprintIn->GeneratedClass))
			{
				continue;
			}

			UEdGraphPin* LegitPin = DiffIt->Pin2;
			// make sure we interpreted which pin is which correctly
			check(LegitPin->LinkedTo.Num() > MalformedPin->LinkedTo.Num());

			for (UEdGraphPin* LinkedPin : LegitPin->LinkedTo)
			{
				// pin linked-to-count diffs always come in pairs (one for the
				// input pin, another for the output)... we use this to know 
				// which pins we should attempt to link again
				TDiffIt const* CorrespendingDiff = PinLinkDiffsForRepair.Find(LinkedPin);
				// we don't have the full pair yet, we'll have to wait till we have the other one
				if (CorrespendingDiff == nullptr)
				{
					continue;
				}

				UEdGraphPin* OtherMalformedPin = (*CorrespendingDiff)->Pin1;
				if (K2Schema->ArePinsCompatible(MalformedPin, OtherMalformedPin, BlueprintIn->GeneratedClass))
				{
					MalformedPin->MakeLinkTo(OtherMalformedPin);
				}
				// else pin types still aren't compatible (even after running 
				// FArchiveReplaceObjectRef), meaning this diff isn't fully resolvable
			}

			// track diffs that are in possible need of repair (so we know which  
			// two pins should attempt to relink)
			PinLinkDiffsForRepair.Add(LegitPin, DiffIt);
		}

		// remove any resolved diffs that no longer are valid (iterating backwards
		// so we can remove array items and not have to offset the index)
		for (int32 DiffIndex = DiffsInOut.Num()-1; DiffIndex >= 0; --DiffIndex)
		{
			FDiffSingleResult const& Diff = DiffsInOut[DiffIndex];
			if ((Diff.Diff == EDiffType::PIN_LINKEDTO_NUM_INC) || (Diff.Diff == EDiffType::PIN_LINKEDTO_NUM_DEC))
			{
				check(Diff.Pin1 && Diff.Pin2);
				// if this diff has been resolved (it's no longer valid)
				if (Diff.Pin1->LinkedTo.Num() == Diff.Pin2->LinkedTo.Num())
				{
					DiffsInOut.RemoveAt(DiffIndex);
				}
			}
		}
	}
};

uint32 FBlueprintAutomationTestUtilities::QueuedTempId = 0u;
TArray<FName> FBlueprintAutomationTestUtilities::DontSavePackagesList;

/************************************************************************/
/* FScopedBlueprintUnloader                                             */
/************************************************************************/

class FScopedBlueprintUnloader
{
public:
	FScopedBlueprintUnloader(bool bAutoOpenScope, bool bRunGCOnCloseIn)
		: bIsOpen(false)
		, bRunGCOnClose(bRunGCOnCloseIn)
	{
		if (bAutoOpenScope)
		{
			OpenScope();
		}
	}

	~FScopedBlueprintUnloader()
	{
		CloseScope();
	}

	/** Tracks currently loaded blueprints at the time of this object's creation */
	void OpenScope()
	{
		PreLoadedBlueprints.Empty();

		// keep a list of blueprints that were loaded at the start (so we can unload new ones after)
		for (TObjectIterator<UBlueprint> BpIt; BpIt; ++BpIt)
		{
			UBlueprint* Blueprint = *BpIt;
			PreLoadedBlueprints.Add(Blueprint);
		}
		bIsOpen = true;
	}

	/** Unloads any blueprints that weren't loaded when this object was created */
	void CloseScope()
	{
		if (bIsOpen)
		{
			// clean up any dependencies that we're loading in the scope of this object's lifetime
			for (TObjectIterator<UBlueprint> BpIt; BpIt; ++BpIt)
			{
				UBlueprint* Blueprint = *BpIt;
				if (PreLoadedBlueprints.Find(Blueprint) == nullptr)
				{
					FBlueprintAutomationTestUtilities::UnloadBlueprint(Blueprint);
				}
			}

			bIsOpen = false;
		}

		// run, even if it was not open (some tests may be relying on this, and 
		// not running it themselves)
		if (bRunGCOnClose)
		{
#if WITH_EDITOR
			// clear undo history to ensure that the transaction buffer isn't 
			// holding onto any references to the blueprints we want unloaded
			GEditor->Trans->Reset(NSLOCTEXT("BpAutomation", "BpAutomationTest", "Blueprint Automation Test"));
#endif // #if WITH_EDITOR
			CollectGarbage(RF_NoFlags);
		}
	}

	void ClearScope()
	{
		PreLoadedBlueprints.Empty();
		bIsOpen = false;
	}

private:
	bool bIsOpen;
	TSet<UBlueprint*> PreLoadedBlueprints;
	bool bRunGCOnClose;
};

/************************************************************************/
/* FCompileBlueprintsTest                                              */
/************************************************************************/

/** Requests a enumeration of all blueprints to be loaded */
void FCompileBlueprintsTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const
{
	FBlueprintAutomationTestUtilities::CollectTestsByClass(UBlueprint::StaticClass(), OutBeautifiedNames, OutTestCommands, /*bool bIgnoreLoaded =*/false);
}


bool FCompileBlueprintsTest::RunTest(const FString& Parameters)
{
	UE_LOGF(LogBlueprintAutomationTests, Log, "Beginning compile test for %ls", *Parameters);
	return FBlueprintAutomationTestUtilities::CompileBlueprint(Parameters);
}

/************************************************************************/
/* FCompileAnimBlueprintsTest                                           */
/************************************************************************/

/** Requests a enumeration of all blueprints to be loaded */
void FCompileAnimBlueprintsTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const
{
	FBlueprintAutomationTestUtilities::CollectTestsByClass(UAnimBlueprint::StaticClass(), OutBeautifiedNames, OutTestCommands, /*bool bIgnoreLoaded =*/false);
}

bool FCompileAnimBlueprintsTest::RunTest(const FString& Parameters)
{
	return FBlueprintAutomationTestUtilities::CompileBlueprint(Parameters);
}

/************************************************************************/
/* FCompileMissingNodeBlueprint                                         */
/************************************************************************/

/** Tests compilation of a Blueprint object that had a missing node from class removal - C12694393 */
bool FCompileMissingNodeBlueprint::RunTest(const FString& Parameters)
{
	return FBlueprintAutomationTestUtilities::CompileBlueprint(TEXT("/EditorTests/Blueprints/BP_NodeClassIsMissing.BP_NodeClassIsMissing"));
}