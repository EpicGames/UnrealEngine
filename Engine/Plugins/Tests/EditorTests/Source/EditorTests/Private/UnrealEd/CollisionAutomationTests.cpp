// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealEd/CollisionAutomationTests.h"
#include "Misc/AutomationTest.h"
#include "EngineGlobals.h"
#include "GameFramework/Actor.h"
#include "CollisionQueryParams.h"
#include "Components/StaticMeshComponent.h"
#include "Components/ShapeComponent.h"
#include "Components/CapsuleComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Editor.h"



#if WITH_EDITOR
#include "Tests/AutomationEditorCommon.h"
#endif

#include "AssetSelection.h"
#include "Engine/TriggerCapsule.h"
#include "Engine/TriggerBox.h"
#include "Engine/TriggerSphere.h"

DEFINE_LOG_CATEGORY_STATIC(CollisionAutomationTestLog, Log, All);


namespace CollisionAutomationTests
{
	FAutomationTestBase* TestBase;
	// Return the currently active world
	UWorld* GetAutomationWorld(const EAutomationTestFlags TestFlags)
 	{
		UWorld* World = nullptr;
		if (!!(TestFlags & EAutomationTestFlags::ClientContext))
		{
			check(GEngine->GetWorldContexts().Num() == 1);
			World = GEngine->GetWorldContexts()[0].World();
		}
		else
		{
			for (auto It = GEngine->GetWorldContexts().CreateConstIterator(); It; ++It)
			{
				const FWorldContext& Context = *It;

				if (Context.WorldType == EWorldType::Editor )
				{
					World = Context.World();
					break;
				}
			}
		}

 		return World;
 	}
	
	// Create a shape mesh actor from a given asset name
	AStaticMeshActor* CreateShapeMeshActor( const FString ShapeAssetName, FVector Location )
	{
		UStaticMesh* StaticMeshAsset = (UStaticMesh*)StaticLoadObject(UStaticMesh::StaticClass(), nullptr, *ShapeAssetName, nullptr, LOAD_None, nullptr);
		TestBase->TestNotNull<UStaticMesh>(FString::Printf(TEXT("Failed to find mesh object %s."), *ShapeAssetName), StaticMeshAsset);

		AStaticMeshActor* ShapeMeshActor = Cast<AStaticMeshActor>(FActorFactoryAssetProxy::AddActorForAsset(StaticMeshAsset));
		ShapeMeshActor->SetActorLocation(Location);
		TestBase->TestNotNull<AActor>(FString::Printf(TEXT("Failed to create MeshActor for %s."), *ShapeAssetName), ShapeMeshActor); 
		return ShapeMeshActor;
	}

	// Create a collision shape actor from a given shape name
	AActor* CreateCollisionShape( UWorld* World, const FString ShapeTypeName, FVector Location )
	{
		AActor* TestRayCollisionActor = nullptr;
		FTransform CollisionTransform;
		CollisionTransform.AddToTranslation(Location);
		
		if (ShapeTypeName == TEXT("TriggerCapsule"))
		{
			TestRayCollisionActor = Cast<ATriggerCapsule>(GEditor->AddActor(World->GetCurrentLevel(), ATriggerCapsule::StaticClass(), CollisionTransform));
			if (TestRayCollisionActor != nullptr)
			{
				UCapsuleComponent* Capsule = Cast<UCapsuleComponent>(TestRayCollisionActor->GetRootComponent());
				if (Capsule != nullptr)
				{
					Capsule->SetCapsuleHalfHeight(100.0f);
					Capsule->SetCapsuleRadius(50.0f);
				}
			}
		}
		else if (ShapeTypeName == TEXT("TriggerBox"))
		{
			TestRayCollisionActor = Cast<ATriggerBox>(GEditor->AddActor(World->GetCurrentLevel(), ATriggerBox::StaticClass(), CollisionTransform));
		}
		else if (ShapeTypeName == TEXT("TriggerSphere"))
		{
			TestRayCollisionActor = Cast<ATriggerSphere>(GEditor->AddActor(World->GetCurrentLevel(), ATriggerSphere::StaticClass(), CollisionTransform));
		}
		TestBase->TestNotNull<AActor>(FString::Printf(TEXT("Failed to create Collision trigger%s."), *ShapeTypeName), TestRayCollisionActor); 
		return TestRayCollisionActor;
	}

	void CheckVector( FVector ResultVector, FVector ExpectedVector, FString TestName, FString ParameterName, int32 TestIndex, float Tolerance = KINDA_SMALL_NUMBER )
	{
		const FVector Delta = ExpectedVector - ResultVector;
		if (Delta.SizeSquared() > FMath::Square(Tolerance))
		{
			//UE_LOGF(CollisionAutomationTestLog, Log, "%d:HitResult=(%ls)", iTest+1, *OutHits[iHits].ToString());
			TestBase->AddError(FString::Printf(TEXT("Test %d:%s %s mismatch. Should be %s but is actually %s."), TestIndex, *TestName, *ParameterName, *ExpectedVector.ToString(), *ResultVector.ToString()));
		}
	}

	void CheckFloat(float ResultFloat, float ExpectedFloat, FString TestName, FString ParameterName, int32 TestIndex, float Tolerance = KINDA_SMALL_NUMBER)
	{
		float Diff = ExpectedFloat - ResultFloat;
		if (Diff > Tolerance)
		{
			//UE_LOGF(CollisionAutomationTestLog, Log, "%d:HitResult=(%ls)", iTest+1, *OutHits[iHits].ToString());
			TestBase->AddError(FString::Printf(TEXT("Test %d:%s %s mismatch. Should be %f but is actually %f."), TestIndex, *TestName, *ParameterName, ExpectedFloat, ResultFloat));
		}
	}

	FString HitToString(FHitResult& HitResult)
	{
		return FString::Printf(TEXT("Time=%f,Location=(X=%f,Y=%f,Z=%f),ImpactPoint=(X=%f,Y=%f,Z=%f),Normal=(X=%f,Y=%f,Z=%f),ImpactNormal=(X=%f,Y=%f,Z=%f),TraceStart=(X=%f,Y=%f,Z=%f),TraceEnd=(X=%f,Y=%f,Z=%f),PenetrationDepth=%f"),
			HitResult.Time,
			HitResult.Location.X,HitResult.Location.Y,HitResult.Location.Z,
			HitResult.ImpactPoint.X,HitResult.ImpactPoint.Y,HitResult.ImpactPoint.Z,
			HitResult.Normal.X,HitResult.Normal.Y,HitResult.Normal.Z,
			HitResult.ImpactNormal.X,HitResult.ImpactNormal.Y,HitResult.ImpactNormal.Z,
			HitResult.TraceStart.X,HitResult.TraceStart.Y,HitResult.TraceStart.Z,
			HitResult.TraceEnd.X,HitResult.TraceEnd.Y,HitResult.TraceEnd.Z,
			HitResult.PenetrationDepth
		);
	}
}

/**
 * ComponentSweepMultiTest Verification
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FComponentSweepMultiTest, "System.Physics.Collision.ComponentSweepMulti", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

/** 
 * Perform some collision sweep tests. Creates a given shape mesh and checks collision normal against a collision shape type.
 * Data for tests is in the [/Script/UnrealEd.CollisionAutomationTestConfigData] section of BaseEditor.ini
 *
 * @param Parameters - Unused for this test
 * @return	TRUE if the test was successful, FALSE otherwise
 */
bool FComponentSweepMultiTest::RunTest(const FString& Parameters)
{	
	CollisionAutomationTests::TestBase = this;
	// Create map
	UWorld* World = FAutomationEditorCommonUtils::CreateNewMap();
	TestNotNull( TEXT("Failed to create world for Physics.Collision.Ray Test. Tests aborted."), World );

	
	FVector StartPos;
	FVector EndPos;
	ECollisionChannel Channel = ECC_WorldStatic;
	
	UCollisionAutomationTestConfigData* Data = UCollisionAutomationTestConfigData::StaticClass()->GetDefaultObject<UCollisionAutomationTestConfigData>();
	
	// Get the tests
	for (int32 iTest = 0; iTest < Data->ComponentSweepMultiTests.Num(); iTest++)
	{
		FCollisionTestEntry OneElement = Data->ComponentSweepMultiTests[iTest];

		// Create the Actor to check against
		AStaticMeshActor* TestRayMeshActor = CollisionAutomationTests::CreateShapeMeshActor( *OneElement.RootShapeAsset, OneElement.HitResult.TraceEnd);
		// Create the collision component
		AActor* TestRayCollisionActor = CollisionAutomationTests::CreateCollisionShape( World, OneElement.ShapeType, OneElement.HitResult.TraceStart);
		
		if ((TestRayMeshActor != nullptr) && (TestRayCollisionActor != nullptr))
		{			
			// Set the collision profile and enable collision and physics
			TestRayMeshActor->GetStaticMeshComponent()->BodyInstance.SetCollisionProfileName(TEXT("BlockAll"));
			TestRayMeshActor->SetActorEnableCollision(true);
			TestRayMeshActor->GetStaticMeshComponent()->BodyInstance.bSimulatePhysics = true;
			UShapeComponent* CollisionComponent = Cast<UShapeComponent>(TestRayCollisionActor->GetRootComponent());
			TestRayCollisionActor->SetActorEnableCollision(true);
			if( CollisionComponent != nullptr )
			{
				CollisionComponent->SetCollisionProfileName(TEXT("BlockAll"));
				CollisionComponent->SetSimulatePhysics(true);
			}
			
			// Setup positions
			StartPos = TestRayCollisionActor->GetActorLocation();
			EndPos = TestRayMeshActor->GetActorLocation();
			// Setup the query
			FComponentQueryParams ShapeQueryParameters(SCENE_QUERY_STAT(TestTrace), nullptr);
			ShapeQueryParameters.bTraceComplex = true;

			// Perform test
			TArray<FHitResult> OutHits;
			bool WasBlocked = World->ComponentSweepMulti(OutHits, CollisionComponent, StartPos, EndPos, FRotator::ZeroRotator, ShapeQueryParameters);
			bool BlockedBySpecified = false;
			if (WasBlocked == true)
			{
				for (int32 iHits = 0; iHits < OutHits.Num(); iHits++)
				{
					AActor* EachActor = OutHits[iHits].GetHitObjectHandle().FetchActor();
					if (EachActor == TestRayMeshActor)
					{
						BlockedBySpecified = true;	
						// This generates a snippet you can copy/paste into the ini file for test validation
						//UE_LOGF(CollisionAutomationTestLog, Log, "%d:HitResult=(%ls)", iTest+1, *(CollisionAutomationTests::HitToString(OutHits[iHits])));

						CollisionAutomationTests::CheckVector( OutHits[iHits].ImpactNormal, OneElement.HitResult.ImpactNormal, TEXT("ComponentSweepMulti"), TEXT("ImpactNormal"), iTest );
						CollisionAutomationTests::CheckVector( OutHits[iHits].Normal, OneElement.HitResult.Normal, TEXT("ComponentSweepMulti"), TEXT("Normal"), iTest );
						CollisionAutomationTests::CheckVector( OutHits[iHits].ImpactPoint, OneElement.HitResult.ImpactPoint, TEXT("ComponentSweepMulti"), TEXT("ImpactPoint"), iTest );
						CollisionAutomationTests::CheckFloat( OutHits[iHits].Time, OneElement.HitResult.Time, TEXT("ComponentSweepMulti"), TEXT("Time"), iTest );
					}
				}
			}
			TestTrue(FString::Printf(TEXT("Test %d:ComponentSweepMulti from %s to %s failed. Should return blocking hit"), iTest+1, *TestRayMeshActor->GetName(), *TestRayCollisionActor->GetName()), BlockedBySpecified);
		}
		// Remove the actors
		TestRayMeshActor->Destroy();
		TestRayCollisionActor->Destroy();
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLineTraceSingleByChannel, "System.Physics.Collision.LineTraceSingleByChannel", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)


/** 
 * Perform LineTraceSingleByChannel tests. Does a ray trace from a given point to a given shape mesh and verifies blocking is correct.
 * Data for tests is in the [/Script/UnrealEd.CollisionAutomationTestConfigData] section of BaseEditor.ini
 *
 * @param Parameters - Unused for this test
 * @return	TRUE if the test was successful, FALSE otherwise
 */
bool FLineTraceSingleByChannel::RunTest(const FString& Parameters)
{	
	// Create map
	UWorld* World = FAutomationEditorCommonUtils::CreateNewMap();
	TestNotNull(TEXT("Failed to create world for Physics.Collision.Ray Test. Tests aborted."), World);
	CollisionAutomationTests::TestBase = this;

	static FName TraceIdent = FName(TEXT("TestTrace"));

	FVector StartPos;
	FVector EndPos;
	ECollisionChannel Channel = ECC_WorldStatic;

	UCollisionAutomationTestConfigData* Data = UCollisionAutomationTestConfigData::StaticClass()->GetDefaultObject<UCollisionAutomationTestConfigData>();

	for (int32 iTest = 0; iTest < Data->LineTraceSingleByChannelTests.Num(); iTest++)
	{
		FCollisionTestEntry OneElement = Data->LineTraceSingleByChannelTests[iTest];

		AStaticMeshActor* TestRayMeshActor = CollisionAutomationTests::CreateShapeMeshActor(*OneElement.RootShapeAsset, OneElement.HitResult.TraceEnd);
		
		if (TestRayMeshActor != nullptr)
		{
			// Create the Actor to check against			
			TestRayMeshActor->GetStaticMeshComponent()->BodyInstance.SetCollisionProfileName(TEXT("BlockAll"));
			TestRayMeshActor->SetActorEnableCollision(true);
			TestRayMeshActor->GetStaticMeshComponent()->BodyInstance.bSimulatePhysics = true;

			// Setup trace start/end
			StartPos = OneElement.HitResult.TraceStart;			
			EndPos = TestRayMeshActor->GetActorLocation();
						
			// Do the trace
			FHitResult OutHit;
			bool WasBlocked = World->LineTraceSingleByChannel(OutHit,StartPos,EndPos,Channel);
			bool BlockedBySpecified = false;
			if (WasBlocked == true)
			{
				if (OutHit.GetHitObjectHandle() == TestRayMeshActor)
				{
					BlockedBySpecified = true;
					// This generates a snippet you can copy/paste into the ini file for test validation
					//UE_LOGF(CollisionAutomationTestLog, Log, "%d:HitResult=(%ls)", iTest + 1, *(CollisionAutomationTests::HitToString(OutHit)));

					CollisionAutomationTests::CheckVector(OutHit.ImpactNormal, OneElement.HitResult.ImpactNormal, TEXT("LineTraceSingleByChannel"), TEXT("ImpactNormal"), iTest);
					CollisionAutomationTests::CheckVector(OutHit.Normal, OneElement.HitResult.Normal, TEXT("LineTraceSingleByChannel"), TEXT("Normal"), iTest);
					CollisionAutomationTests::CheckVector(OutHit.ImpactPoint, OneElement.HitResult.ImpactPoint, TEXT("LineTraceSingleByChannel"), TEXT("ImpactPoint"), iTest);
					CollisionAutomationTests::CheckFloat(OutHit.Time, OneElement.HitResult.Time, TEXT("LineTraceSingleByChannel"), TEXT("Time"), iTest);									
				}
			}
			TestTrue(FString::Printf(TEXT("Test %d:LineTraceSingleByChannel to %s failed. Should return blocking hit"), iTest + 1, *TestRayMeshActor->GetName()), BlockedBySpecified);

			// Change the collision profile and ensure we dont get a blocking hit
			UShapeComponent* CollisionComponent = Cast<UShapeComponent>(TestRayMeshActor->GetRootComponent());			
			if (CollisionComponent != nullptr)
			{
				CollisionComponent->SetCollisionProfileName(TEXT("OverlapAll"));
				CollisionComponent->SetSimulatePhysics(true);
			}
				
			TestRayMeshActor->GetStaticMeshComponent()->BodyInstance.SetCollisionProfileName(TEXT("OverlapAll"));
			WasBlocked = World->LineTraceSingleByChannel(OutHit, StartPos, EndPos, Channel);
			TestFalse(FString::Printf(TEXT("Test %d:LineTraceSingleByChannel to %s failed. Should not return blocking hit"), iTest + 1, *TestRayMeshActor->GetName()), WasBlocked);			
		}
		// Remove the actor
		TestRayMeshActor->Destroy();
	}

	return true;
}

