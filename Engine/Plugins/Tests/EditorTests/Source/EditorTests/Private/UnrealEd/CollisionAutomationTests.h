// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/HitResult.h"
#include "CollisionAutomationTests.generated.h"

/**
 * Container for detailing collision automated test data.
 */
USTRUCT()
struct FCollisionTestEntry
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString RootShapeAsset;

	UPROPERTY()
	FString ShapeType;

	UPROPERTY()
	FHitResult	HitResult;

	FCollisionTestEntry()
	{
	}
};

USTRUCT()
struct FCollisionPerfTest
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString RootShapeAsset;

	UPROPERTY()
	FString ShapeType;

	UPROPERTY()
	FVector CreationBounds;

	UPROPERTY()
	FVector CreationElements;

	FCollisionPerfTest()
		: CreationBounds(ForceInitToZero)
		, CreationElements(ForceInitToZero)
	{
	}
};

UCLASS(config=Editor)
class UCollisionAutomationTestConfigData : public UObject
{
public:
	GENERATED_BODY()
		
	UPROPERTY(config)
	TArray<FCollisionTestEntry>	ComponentSweepMultiTests;
	UPROPERTY(config)
	TArray<FCollisionTestEntry>	LineTraceSingleByChannelTests;
	
	UPROPERTY(config)
	TArray<FCollisionPerfTest>	LineTracePerformanceTests;
};
