// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/AutomationTest.h"
#include "Modules/ModuleManager.h"
#include "UObject/GCObjectScopeGuard.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/SWidget.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/Blueprint.h"
#include "GameFramework/Actor.h"
#include "Animation/Skeleton.h"
#include "Curves/CurveBase.h"
#include "Curves/CurveFloat.h"
#include "GameFramework/ForceFeedbackAttenuation.h"
#include "GameFramework/ForceFeedbackEffect.h"
#include "Sound/SoundAttenuation.h"
#include "Animation/AnimMontage.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "Particles/ParticleSystem.h"
#include "Materials/MaterialFunction.h"
#include "Materials/Material.h"
#include "Sound/SoundClass.h"
#include "Factories/Factory.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/Font.h"
#include "Engine/DataAsset.h"
#include "Animation/AnimBlueprint.h"
#include "Sound/DialogueVoice.h"
#include "Sound/DialogueWave.h"
#include "Exporters/Exporter.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialParameterCollection.h"
#include "Engine/ObjectLibrary.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "Sound/ReverbEffect.h"
#include "Engine/Selection.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundMix.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTargetCube.h"
#include "GameFramework/TouchInterface.h"
#include "Engine/UserDefinedEnum.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Animation/AnimInstance.h"
#include "Editor/UnrealEdEngine.h"
#include "Factories/AnimBlueprintFactory.h"
#include "Factories/AnimCompositeFactory.h"
#include "Factories/AnimMontageFactory.h"
#include "Factories/BlendSpaceFactory1D.h"
#include "Factories/AimOffsetBlendSpaceFactory1D.h"
#include "Factories/BlendSpaceFactoryNew.h"
#include "Factories/AimOffsetBlendSpaceFactoryNew.h"
#include "Factories/BlueprintFactory.h"
#include "Factories/BlueprintFunctionLibraryFactory.h"
#include "Factories/BlueprintMacroFactory.h"
#include "Factories/DialogueVoiceFactory.h"
#include "Factories/DialogueWaveFactory.h"
#include "Factories/EnumFactory.h"
#include "Factories/ForceFeedbackAttenuationFactory.h"
#include "Factories/ForceFeedbackEffectFactory.h"
#include "Factories/MaterialFactoryNew.h"
#include "Factories/MaterialFunctionFactoryNew.h"
#include "Factories/MaterialInstanceConstantFactoryNew.h"
#include "Factories/MaterialParameterCollectionFactoryNew.h"
#include "Factories/ObjectLibraryFactory.h"
#include "Factories/ParticleSystemFactoryNew.h"
#include "Factories/PhysicalMaterialFactoryNew.h"
#include "Factories/ReverbEffectFactory.h"
#include "Factories/SlateBrushAssetFactory.h"
#include "Factories/SlateWidgetStyleAssetFactory.h"
#include "Factories/SoundAttenuationFactory.h"
#include "Factories/SoundClassFactory.h"
#include "Factories/SoundCueFactoryNew.h"
#include "Factories/SoundMixFactory.h"
#include "Factories/StructureFactory.h"
#include "Factories/TrueTypeFontFactory.h"
#include "Factories/TextureRenderTargetCubeFactoryNew.h"
#include "Factories/TextureRenderTargetFactoryNew.h"
#include "Factories/TouchInterfaceFactory.h"
#include "Editor.h"
#include "UnrealEdGlobals.h"
#include "UObject/SavePackage.h"

#include "Tests/AutomationTestSettings.h"
#include "ObjectTools.h"
#include "PackageTools.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include "ContentBrowserModule.h"

#include "Slate/SlateBrushAsset.h"
#include "Framework/Styling/ButtonWidgetStyle.h"

#include "Tests/AutomationCommon.h"
#include "Tests/AutomationEditorCommon.h"
#include "Animation/BlendSpace.h"
#include "Animation/AimOffsetBlendSpace.h"
#include "Animation/BlendSpace1D.h"
#include "Animation/AimOffsetBlendSpace1D.h"
#include "Animation/AnimComposite.h"
#include "Factories/DataAssetFactory.h"
#include "Factories/CurveFactory.h"
#include "AssetExportTask.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "EditorAssetAutomationTests"

DEFINE_LOG_CATEGORY_STATIC(LogEditorAssetAutomationTests, Log, All);

/**
* Container for items related to the create asset test
*/
namespace CreateAssetHelper
{
	/**
	* Gets the base path for this asset
	*/
	static FString GetGamePath()
	{
		return TEXT("/Temp/Automation/Transient/Automation_AssetCreationDuplication");
	}

	/**
	* Gets the full path to the folder on disk
	*/
	static FString GetFullPath()
	{
		return FPackageName::FilenameToLongPackageName(FPaths::AutomationTransientDir() + TEXT("Automation_AssetCreationDuplication"));
	}

	/** 
	* Tracks success counts for each stage of the create / duplicate asset test
	*/
	struct FCreateAssetStats
	{
		//Total number of assets
		uint32 NumTotalAssets;

		//Number of assets skipped
		uint32 NumSkippedAssets;

		//Number of assets created
		uint32 NumCreated;

		//Number of assets saved
		uint32 NumSaved;

		//Number of duplicates saved
		uint32 NumDuplicatesSaved;

		//Number of assets duplicated
		uint32 NumDuplicated;

		//Number of assets deleted
		uint32 NumDeleted;

		/**
		* Constructor
		*/
		FCreateAssetStats() :
			NumTotalAssets(0),
			NumSkippedAssets(0),
			NumCreated(0),
			NumSaved(0),
			NumDuplicatesSaved(0),
			NumDuplicated(0),
			NumDeleted(0)
		{
		}
	};

	/**
	* Handles creating, duplicating, and saving assets
	*/
	struct FCreateAssetInfo
	{
		//The name to use for this asset
		FString AssetName;

		//The location this asset will be created at
		FString AssetPath;

		//The class of the asset
		UClass* Class;

		//The factory to use to create this asset
		UFactory* Factory;

		//The asset that was created
		UObject* CreatedAsset;

		//The package that contains the asset
		UPackage* AssetPackage;

		//The duplicated asset
		UObject* DuplicatedAsset;

		//The package that contains the duplicated asset
		UPackage* DuplicatedPackage;

		//Pointer to the asset test stats
		TSharedPtr<FCreateAssetStats> TestStats;

		/**
		* Constructor
		*/
		FCreateAssetInfo(const FString& InAssetName, const FString& InAssetPath, UClass* InClass, UFactory* InFactory, TSharedPtr<FCreateAssetStats> InStats) :
			AssetName(InAssetName),
			AssetPath(InAssetPath),
			Class(InClass),
			Factory(InFactory),
			CreatedAsset(NULL),
			AssetPackage(NULL),
			DuplicatedAsset(NULL),
			DuplicatedPackage(NULL),
			TestStats(InStats)
		{
		}

		/**
		* Creates the new item
		*/
		void CreateAsset()
		{
			const FString PackageName = AssetPath + TEXT("/") + AssetName;
			AssetPackage = CreatePackage(*PackageName);
			EObjectFlags Flags = RF_Public | RF_Standalone;

			CreatedAsset = Factory->FactoryCreateNew(Class, AssetPackage, FName(*AssetName), Flags, NULL, GWarn);

			if (CreatedAsset)
			{
				// Notify the asset registry
				FAssetRegistryModule::AssetCreated(CreatedAsset);

				// Mark the package dirty...
				AssetPackage->MarkPackageDirty();

				TestStats->NumCreated++;
				UE_LOGF(LogEditorAssetAutomationTests, Display, "Created asset %ls (%ls)", *AssetName, *Class->GetName());
			}
			else
			{
				UE_LOGF(LogEditorAssetAutomationTests, Error, "Unable to create asset of type %ls", *Class->GetName());
			}
		}

		/**
		* Saves the created asset
		*/
		void SaveNewAsset()
		{
			if (AssetPackage && CreatedAsset)
			{
				AssetPackage->SetDirtyFlag(true);
				const FString PackagePath = FString::Printf(TEXT("%s/%s"), *GetGamePath(), *AssetName);
				FSavePackageArgs SaveArgs;
				SaveArgs.TopLevelFlags = RF_Standalone;
				SaveArgs.SaveFlags = SAVE_NoError;
				if (UPackage::SavePackage(AssetPackage, nullptr,
					*FPackageName::LongPackageNameToFilename(PackagePath, FPackageName::GetAssetPackageExtension()),
					SaveArgs))
				{
					TestStats->NumSaved++;
					UE_LOGF(LogEditorAssetAutomationTests, Display, "Saved asset %ls (%ls)", *CreatedAsset->GetName(), *Class->GetName());
				}
				else
				{
					UE_LOGF(LogEditorAssetAutomationTests, Display, "Unable to save asset %ls (%ls)", *CreatedAsset->GetName(), *Class->GetName());
				}
			}
		}

		/**
		* Saves the duplicated asset
		*/
		void SaveDuplicatedAsset()
		{
			if (DuplicatedPackage && DuplicatedAsset)
			{
				DuplicatedPackage->SetDirtyFlag(true);
				const FString PackagePath = FString::Printf(TEXT("%s/%s_Copy"), *GetGamePath(), *AssetName);
				FSavePackageArgs SaveArgs;
				SaveArgs.TopLevelFlags = RF_Standalone;
				SaveArgs.SaveFlags = SAVE_NoError;
				if (UPackage::SavePackage(DuplicatedPackage, nullptr,
					*FPackageName::LongPackageNameToFilename(PackagePath, FPackageName::GetAssetPackageExtension()),
					SaveArgs))
				{
					TestStats->NumDuplicatesSaved++;
					UE_LOGF(LogEditorAssetAutomationTests, Display, "Saved asset %ls (%ls)", *DuplicatedAsset->GetName(), *Class->GetName());
				}
				else
				{
					UE_LOGF(LogEditorAssetAutomationTests, Display, "Unable to save asset %ls (%ls)", *DuplicatedAsset->GetName(), *Class->GetName());
				}
			}
		}

		/**
		* Duplicates the asset
		*/
		void DuplicateAsset()
		{
			if (AssetPackage && CreatedAsset)
			{
				const FString NewObjectName = FString::Printf(TEXT("%s_Copy"), *AssetName);
				const FString NewPackageName = FString::Printf(TEXT("%s/%s"), *GetGamePath(), *NewObjectName);

				// Make sure the referenced object is deselected before duplicating it.
				GEditor->GetSelectedObjects()->Deselect(CreatedAsset);

				// Duplicate the asset
				DuplicatedPackage = CreatePackage(*NewPackageName);
				DuplicatedAsset = StaticDuplicateObject(CreatedAsset, DuplicatedPackage, *NewObjectName);

				if (DuplicatedAsset)
				{
					DuplicatedAsset->MarkPackageDirty();

					// Notify the asset registry
					FAssetRegistryModule::AssetCreated(DuplicatedAsset);

					TestStats->NumDuplicated++;
					UE_LOGF(LogEditorAssetAutomationTests, Display, "Duplicated asset %ls to %ls (%ls)", *AssetName, *NewObjectName, *Class->GetName());
				}
				else
				{
					UE_LOGF(LogEditorAssetAutomationTests, Error, "Failed to duplicate asset %ls (%ls)", *AssetName, *Class->GetName());
				}
			}
		}

		/**
		* Deletes the asset
		*/
		void DeleteAsset()
		{
			if (CreatedAsset)
			{
				bool bSuccessful = false;

				bSuccessful = ObjectTools::DeleteSingleObject(CreatedAsset, false);

				//If we failed to delete this object manually clear any references and try again
				if (!bSuccessful)
				{
					//Clear references to the object so we can delete it
					FAutomationEditorCommonUtils::NullReferencesToObject(CreatedAsset);

					bSuccessful = ObjectTools::DeleteSingleObject(CreatedAsset, false);
				}

				//Delete the package
				if (bSuccessful)
				{
					FString PackageFilename;
					if (FPackageName::DoesPackageExist(AssetPackage->GetName(), &PackageFilename))
					{
						TArray<UPackage*> PackagesToDelete;
						PackagesToDelete.Add(AssetPackage);
						// Let the package auto-saver know that it needs to ignore the deleted packages
						GUnrealEd->GetPackageAutoSaver().OnPackagesDeleted(PackagesToDelete);

						AssetPackage->SetDirtyFlag(false);

						// Unload the packages and collect garbage.
						UPackageTools::UnloadPackages(PackagesToDelete);

						IFileManager::Get().Delete(*PackageFilename);

						TestStats->NumDeleted++;
						UE_LOGF(LogEditorAssetAutomationTests, Display, "Deleted asset %ls (%ls)", *AssetName, *Class->GetName());
					}
				}
				else
				{
					UE_LOGF(LogEditorAssetAutomationTests, Error, "Unable to delete asset: %ls (%ls)", *AssetName, *Class->GetName());
				}
			}
		}
	};
}

/**
* Latent command to create an asset
*/
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FCreateNewAssetCommand, TSharedPtr<CreateAssetHelper::FCreateAssetInfo>, AssetInfo);
bool FCreateNewAssetCommand::Update()
{
	AssetInfo->CreateAsset();
	return true;
}


/**
* Latent command to save an asset
*/
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FSaveNewAssetCommand, TSharedPtr<CreateAssetHelper::FCreateAssetInfo>, AssetInfo);
bool FSaveNewAssetCommand::Update()
{
	AssetInfo->SaveNewAsset();
	return true;
}

/**
* Latent command to save a duplicated asset
*/
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FSaveDuplicateAssetCommand, TSharedPtr<CreateAssetHelper::FCreateAssetInfo>, AssetInfo);
bool FSaveDuplicateAssetCommand::Update()
{
	AssetInfo->SaveDuplicatedAsset();
	return true;
}

/**
* Latent command to duplicate an asset
*/
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FDuplicateAssetCommand, TSharedPtr<CreateAssetHelper::FCreateAssetInfo>, AssetInfo);
bool FDuplicateAssetCommand::Update()
{
	AssetInfo->DuplicateAsset();
	return true;
}

/**
* Latent command to delete an asset
*/
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FDeleteAssetCommand, TSharedPtr<CreateAssetHelper::FCreateAssetInfo>, AssetInfo);
bool FDeleteAssetCommand::Update()
{
	AssetInfo->DeleteAsset();
	return true;
}

/**
* Latent command to clear editor references to temporary objects
*/
DEFINE_LATENT_AUTOMATION_COMMAND(FClearEditorReferencesCommand);
bool FClearEditorReferencesCommand::Update()
{
	// Deselect all
	GEditor->SelectNone(false, true);

	// Clear the transaction buffer so we aren't referencing the new objects
	GUnrealEd->ResetTransaction(FText::FromString(TEXT("FAssetEditorTest")));

	return true;
}

/**
* Latent command to disable the behavior tree editor
*/
DEFINE_LATENT_AUTOMATION_COMMAND(FDisableBehaviorTreeEditorCommand);
bool FDisableBehaviorTreeEditorCommand::Update()
{
	bool bEnableBehaviorTrees = false;
	GConfig->SetBool(TEXT("BehaviorTreesEd"), TEXT("BehaviorTreeNewAssetsEnabled"), bEnableBehaviorTrees, GEngineIni);
	return true;
}

/**
* Latent command log the asset creation stats
*/
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FLogAssetCreationStatsCommand, TSharedPtr<CreateAssetHelper::FCreateAssetStats>, BuildStats);
bool FLogAssetCreationStatsCommand::Update()
{
	UE_LOGF(LogEditorAssetAutomationTests, Display, " ");
	UE_LOGF(LogEditorAssetAutomationTests, Display, "Test Summary:");
	if (BuildStats->NumSkippedAssets)
	{
		UE_LOGF(LogEditorAssetAutomationTests, Display, "Skipped %i assets", BuildStats->NumSkippedAssets);
	}
	UE_LOGF(LogEditorAssetAutomationTests, Display, "%i of %i assets were created successfully",			BuildStats->NumCreated,			BuildStats->NumTotalAssets);
	UE_LOGF(LogEditorAssetAutomationTests, Display, "%i of %i assets were saved successfully",				BuildStats->NumSaved,			BuildStats->NumTotalAssets);
	UE_LOGF(LogEditorAssetAutomationTests, Display, "%i of %i assets were duplicated successfully",		BuildStats->NumDuplicated,		BuildStats->NumTotalAssets);
	UE_LOGF(LogEditorAssetAutomationTests, Display, "%i of %i duplicated assets were saved successfully",	BuildStats->NumDuplicatesSaved, BuildStats->NumTotalAssets);
	UE_LOGF(LogEditorAssetAutomationTests, Display, "%i of %i assets were deleted successfully",			BuildStats->NumDeleted,			BuildStats->NumTotalAssets);
	return true;
}

template<typename TAssetClass, typename TFactoryClass, typename ExtraCommandsFunc>
void AssetTestCreate(const TCHAR* NamePrefix, const FString& CurrentTimestamp, const FString& GamePath, TArray< TSharedPtr<CreateAssetHelper::FCreateAssetInfo> >& AssetInfos, const TSharedPtr<CreateAssetHelper::FCreateAssetStats>& BuildStats, ExtraCommandsFunc ExtraCommands)
{
	FString NameString = FString::Printf(TEXT("%s_%s"), NamePrefix, *CurrentTimestamp);
	TFactoryClass* FactoryInst = NewObject<TFactoryClass>();
	ExtraCommands(FactoryInst);
	TSharedPtr<CreateAssetHelper::FCreateAssetInfo> CreateInfo = MakeShareable(new CreateAssetHelper::FCreateAssetInfo(NameString, GamePath, TAssetClass::StaticClass(), FactoryInst, BuildStats));
	AssetInfos.Add(CreateInfo);
	ADD_LATENT_AUTOMATION_COMMAND(FCreateNewAssetCommand(CreateInfo));
}

//Macro to create a factory and queue up a command to create a given asset type
#define ASSET_TEST_CREATE( TAssetClass, TFactoryClass, NamePrefix, ExtraCommands ) \
	AssetTestCreate<TAssetClass, TFactoryClass>(TEXT(#NamePrefix), CurrentTimestamp, GamePath, AssetInfos, BuildStats, [=](TFactoryClass* FactoryInst) { ExtraCommands });

//Macro to create a factory by name and queue up a command to create an asset
#define ASSET_TEST_CREATE_BY_NAME(TAssetClassName, TFactoryClassName, NamePrefix, ExtraCommands) \
{ \
	FString NameString = FString::Printf(TEXT("%s_%s"), TEXT(#NamePrefix), *CurrentTimestamp); \
	UClass* FactoryClass = StaticLoadClass(UFactory::StaticClass(), NULL, TEXT(#TFactoryClassName), NULL, LOAD_None, NULL); \
	if (FactoryClass) \
	{ \
		UFactory* FactoryInst = NewObject<UFactory>(GetTransientPackage(), FactoryClass); \
		ExtraCommands \
		UClass* AssetClass = StaticLoadClass(UObject::StaticClass(), NULL, TEXT(#TAssetClassName), NULL, LOAD_None, NULL); \
		TSharedPtr<CreateAssetHelper::FCreateAssetInfo> CreateInfo = MakeShareable(new CreateAssetHelper::FCreateAssetInfo(NameString, GamePath, AssetClass, FactoryInst, BuildStats)); \
		AssetInfos.Add(CreateInfo); \
		ADD_LATENT_AUTOMATION_COMMAND(FCreateNewAssetCommand(CreateInfo)); \
	} \
	else \
	{ \
		UE_LOGF(LogEditorAssetAutomationTests,Error,"Couldn't find factory class %ls",TEXT(#TFactoryClassName)); \
	} \
}


/**
* Automation test for creating, saving, and duplicating assets
*/
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAssetEditorTest, "Editor.Content.Asset Creation and Duplication", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FAssetEditorTest::RunTest(const FString& Parameters)
{
	// todo - doesn't appear to work with userdir arg
#if 0
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	const FString FullPath = CreateAssetHelper::GetFullPath();
	const FString GamePath = CreateAssetHelper::GetGamePath();

	//Create the folder if it doesn't already exist
	if (!IFileManager::Get().DirectoryExists(*FullPath))
	{
		//Make the new folder
		IFileManager::Get().MakeDirectory(*FullPath, true);

		// Add the path to the asset registry
		AssetRegistryModule.Get().AddPath(GamePath);

		// Notify 'asset path changed' delegate
		FContentBrowserModule::FOnAssetPathChanged& PathChangedDelegate = ContentBrowserModule.GetOnAssetPathChanged();
		if (PathChangedDelegate.IsBound())
		{
			PathChangedDelegate.Broadcast(GamePath);
		}
	}

	//Timestamp
	FString CurrentTimestamp = FPlatformTime::StrTimestamp();
	CurrentTimestamp = CurrentTimestamp.Replace(TEXT("/"), TEXT(""));
	CurrentTimestamp = CurrentTimestamp.Replace(TEXT(":"), TEXT(""));
	CurrentTimestamp = CurrentTimestamp.Replace(TEXT(" "), TEXT("_"));

	//Skeleton - Grab the first available loaded skeleton
	TArray<FAssetData> AllSkeletons;
	AssetRegistryModule.Get().GetAssetsByClass(USkeleton::StaticClass()->GetFName(), AllSkeletons);
	USkeleton* FirstSkeleton = NULL;
	for (int32 SkelIndex = 0; SkelIndex < AllSkeletons.Num(); ++SkelIndex)
	{
		if (AllSkeletons[SkelIndex].IsAssetLoaded())
		{
			FirstSkeleton = (USkeleton*)(AllSkeletons[SkelIndex].GetAsset());
			break;
		}
	}

	// No skeleton was loaded, just load one the first one found
	// This is only used to verify we can create assets that rely on skeletons.
	if (!FirstSkeleton && AllSkeletons.Num())
	{
		FirstSkeleton = CastChecked<USkeleton>(AllSkeletons[0].GetAsset());
	}

	//Check to see if we need to enable Behavior trees.
	bool bEnabledBehaviorTrees = false;
	bool bBehaviorTreeNewAssetsEnabled = false;
	GConfig->GetBool(TEXT("BehaviorTreesEd"), TEXT("BehaviorTreeNewAssetsEnabled"), bBehaviorTreeNewAssetsEnabled, GEngineIni);
	if (!bBehaviorTreeNewAssetsEnabled)
	{
		bEnabledBehaviorTrees = true;
		GConfig->SetBool(TEXT("BehaviorTreesEd"), TEXT("BehaviorTreeNewAssetsEnabled"), bEnabledBehaviorTrees, GEngineIni);

		if (!FModuleManager::Get().IsModuleLoaded(TEXT("BehaviorTreeEditor")))
		{
			//NOTE: This module gets left in after the test completes otherwise the content browser would crash when it tries to access the created BehaviorTree.
			FModuleManager::Get().LoadModule(TEXT("BehaviorTreeEditor"));
		}
	}

	//Holds info on each asset we are creating
	TArray< TSharedPtr<CreateAssetHelper::FCreateAssetInfo> > AssetInfos;
	TSharedPtr<CreateAssetHelper::FCreateAssetStats> BuildStats = (MakeShareable(new CreateAssetHelper::FCreateAssetStats()));

	//Queue creating the different kinds of assets
	ASSET_TEST_CREATE(UBlueprint, UBlueprintFactory, BP, FactoryInst->ParentClass = AActor::StaticClass();)
	ASSET_TEST_CREATE(UMaterial, UMaterialFactoryNew, MAT, )
	ASSET_TEST_CREATE(UParticleSystem, UParticleSystemFactoryNew, PS, )
	
	if (FirstSkeleton)
	{
		ASSET_TEST_CREATE(UAimOffsetBlendSpace, UAimOffsetBlendSpaceFactoryNew, AO, FactoryInst->TargetSkeleton = FirstSkeleton;)
		ASSET_TEST_CREATE(UAimOffsetBlendSpace1D, UAimOffsetBlendSpaceFactory1D, AO1D, FactoryInst->TargetSkeleton = FirstSkeleton;)
		ASSET_TEST_CREATE(UAnimBlueprint, UAnimBlueprintFactory, AB, FactoryInst->ParentClass = UAnimInstance::StaticClass(); FactoryInst->TargetSkeleton = FirstSkeleton;)
		ASSET_TEST_CREATE(UAnimComposite, UAnimCompositeFactory, AC, FactoryInst->TargetSkeleton = FirstSkeleton;)
		ASSET_TEST_CREATE(UAnimMontage, UAnimMontageFactory, AM, FactoryInst->TargetSkeleton = FirstSkeleton;)
		ASSET_TEST_CREATE(UBlendSpace, UBlendSpaceFactoryNew, BS, FactoryInst->TargetSkeleton = FirstSkeleton;)
		ASSET_TEST_CREATE(UBlendSpace1D, UBlendSpaceFactory1D, BS1D, FactoryInst->TargetSkeleton = FirstSkeleton;)
	}
	else
	{
		BuildStats->NumSkippedAssets += 7;
		UE_LOGF(LogEditorAssetAutomationTests, Warning, "NO AVAILABLE SKELETON.  Skipping related assets.");
	}

	ASSET_TEST_CREATE(UTextureRenderTargetCube, UTextureRenderTargetCubeFactoryNew, CRT, )
	ASSET_TEST_CREATE(UFont, UTrueTypeFontFactory, F, )
	ASSET_TEST_CREATE(UMaterialFunction, UMaterialFunctionFactoryNew, MF, )
	ASSET_TEST_CREATE(UMaterialInstanceConstant, UMaterialInstanceConstantFactoryNew, MI, )
	ASSET_TEST_CREATE(UMaterialParameterCollection, UMaterialParameterCollectionFactoryNew, MPC, )
	ASSET_TEST_CREATE(UTextureRenderTarget2D, UTextureRenderTargetFactoryNew, RT, )
	ASSET_TEST_CREATE(UDialogueVoice, UDialogueVoiceFactory, DV, )
	ASSET_TEST_CREATE(UDialogueWave, UDialogueWaveFactory, DW, )
	ASSET_TEST_CREATE(UReverbEffect, UReverbEffectFactory, RE, )
	ASSET_TEST_CREATE(UForceFeedbackAttenuation, UForceFeedbackAttenuationFactory, FFA, )
	ASSET_TEST_CREATE(USoundAttenuation, USoundAttenuationFactory, SA, )
	ASSET_TEST_CREATE(USoundClass, USoundClassFactory, SC, )
	ASSET_TEST_CREATE(USoundCue, USoundCueFactoryNew, Scue, )
	ASSET_TEST_CREATE(USoundMix, USoundMixFactory, SM, )
	ASSET_TEST_CREATE(UPhysicalMaterial, UPhysicalMaterialFactoryNew, PM, )
	ASSET_TEST_CREATE(USlateBrushAsset, USlateBrushAssetFactory, SB, )
	ASSET_TEST_CREATE(USlateWidgetStyleAsset, USlateWidgetStyleAssetFactory, SWS, FactoryInst->StyleType = UButtonWidgetStyle::StaticClass();)
	ASSET_TEST_CREATE_BY_NAME(AIModule.BehaviorTree, BehaviorTreeEditor.BehaviorTreeFactory, BT, )
	ASSET_TEST_CREATE(UBlueprint, UBlueprintFunctionLibraryFactory, BFL, )
	ASSET_TEST_CREATE(UBlueprint, UBlueprintMacroFactory, MPL, FactoryInst->ParentClass = AActor::StaticClass();)
	ASSET_TEST_CREATE(UCurveBase, UCurveFactory, C, FactoryInst->CurveClass = UCurveFloat::StaticClass();)
	UClass* GameplayAbilityClass = StaticLoadClass(UObject::StaticClass(), NULL, TEXT("GameplayAbilities.GameplayAbilitySet"), NULL, LOAD_None, NULL);
	if (GameplayAbilityClass)
	{
		ASSET_TEST_CREATE(UDataAsset, UDataAssetFactory, DA, FactoryInst->DataAssetClass = GameplayAbilityClass;)
	}
	else
	{
		BuildStats->NumSkippedAssets += 1;
		UE_LOGF(LogEditorAssetAutomationTests, Warning, "COULD NOT LOAD GameplayAbilitySet.  Skipping DataAsset creation.");
	}
	
	ASSET_TEST_CREATE(UUserDefinedEnum, UEnumFactory, Enum, )
	ASSET_TEST_CREATE(UForceFeedbackEffect, UForceFeedbackEffectFactory, FFE, )
	ASSET_TEST_CREATE(UInterpData, UInterpDataFactoryNew, MD, )
	ASSET_TEST_CREATE(UObjectLibrary, UObjectLibraryFactory, OL, )
	ASSET_TEST_CREATE(UUserDefinedStruct, UStructureFactory, S, )
	ASSET_TEST_CREATE(UTouchInterface, UTouchInterfaceFactory, TIS, )

	//Record how many assets we are testing
	BuildStats->NumTotalAssets = AssetInfos.Num();

	//Save new assets
	for (int32 i = 0; i < AssetInfos.Num(); i++)
	{
		ADD_LATENT_AUTOMATION_COMMAND(FSaveNewAssetCommand(AssetInfos[i]));
	}

	//Duplicate new assets
	for (int32 i = 0; i < AssetInfos.Num(); i++)
	{
		ADD_LATENT_AUTOMATION_COMMAND(FDuplicateAssetCommand(AssetInfos[i]));
	}

	//Save duplicates
	for (int32 i = 0; i < AssetInfos.Num(); i++)
	{
		ADD_LATENT_AUTOMATION_COMMAND(FSaveDuplicateAssetCommand(AssetInfos[i]));
	}

	ADD_LATENT_AUTOMATION_COMMAND(FClearEditorReferencesCommand());

	//Delete Original
	for (int32 i = 0; i < AssetInfos.Num(); i++)
	{
		ADD_LATENT_AUTOMATION_COMMAND(FDeleteAssetCommand(AssetInfos[i]));
	}

	ADD_LATENT_AUTOMATION_COMMAND(FLogAssetCreationStatsCommand(BuildStats));

	//Disable the behavior trees if we enabled them earlier
	if (bEnabledBehaviorTrees)
	{
		ADD_LATENT_AUTOMATION_COMMAND(FDisableBehaviorTreeEditorCommand());
	}
#endif
	return true;
}

#undef ASSET_TEST_CREATE
#undef ASSET_TEST_CREATE_BY_NAME

/**
* Namespace for helper items for the import / export asset test
*/
namespace ImportExportAssetHelper
{
	// How long to wait for the asset editor window to open.
	static int32 MaxWaitForEditorTicks = 5;

	//State flags for the FAssetInfo class
	namespace EState
	{
		enum Type
		{
			Import,
			OpenEditor,
			WaitForEditor,
			Screenshot,
			Export,
			Done
		};
	}

	/**
	* Import test report for a single asset
	*/
	struct FAssetImportReport
	{
		//The Asset file name
		FString AssetName;

		//If the asset imported successfully
		bool bImportSuccessful;

		//If the export step was skipped
		bool bSkippedExport;

		//If the asset exported successfully
		bool bExportSuccessful;

		//The size of the exported file
		int32 FileSize;

		/**
		* Constructor
		*/
		FAssetImportReport() :
			AssetName(),
			bImportSuccessful(false),
			bSkippedExport(false),
			bExportSuccessful(false),
			FileSize(0)
		{
		}
	};

	/**
	* Import test stats
	*/
	struct FAssetImportStats
	{
		//List of import reports
		TArray<FAssetImportReport> Reports;
	};

	/**
	* Namespace for helper items for the import / export asset test
	*/
	struct FAssetInfo
	{
		//Path to the file we are importing
		FString ImportPath;
		
		//The file extension to use when exporting this asset
		FString ExportExtension;

		//The current state this asset is in
		EState::Type State;

		//A pointer to the asset we imported
		UObject* ImportedAsset;

		//A list of custom settings to apply to our import factory
		TArray<FImportFactorySettingValues> FactorySettings;

		//How many frames we have waited for the asset editor
		int32 WaitingForEditorCount;

		//If we should skip the export step
		bool bSkipExport;

		//Pointer to the execution info of this test
		FAutomationTestExecutionInfo* TestExecutionInfo;

		//Shared list of test results
		TSharedPtr<FAssetImportStats> TestStats;

		//Test report for this asset
		FAssetImportReport TestReport;

		/**
		* Constructor
		*/
		FAssetInfo(const FEditorImportExportTestDefinition& InTestDef, FAutomationTestExecutionInfo* InExecutionInfo, TSharedPtr<FAssetImportStats> InStats) :
			ImportPath(InTestDef.ImportFilePath.FilePath),
			ExportExtension(InTestDef.ExportFileExtension),
			State(EState::Import),
			ImportedAsset(NULL),
			FactorySettings(InTestDef.FactorySettings),
			WaitingForEditorCount(0),
			bSkipExport(InTestDef.bSkipExport),
			TestExecutionInfo(InExecutionInfo),
			TestStats(InStats),
			TestReport()
		{
		}

		/**
		* Updates the import state
		*/
		bool Update()
		{
			switch (State)
			{
			case EState::Import:
				ImportAsset();
				break;
			case EState::OpenEditor:
				OpenEditor();
				break;
			case EState::WaitForEditor:
				CheckEditor();
				break;
			case EState::Screenshot:
				TakeScreenshot();
				break;
			case EState::Export:
				ExportAsset();
				break;
			default:
				break;
			}

			//Clean up the asset if we are done
			if (State == EState::Done)
			{
				if (ImportedAsset)
				{
					DeleteAsset();
				}

				//Report the result
				TestStats->Reports.Add(TestReport);

				return true;
			}

			return false;
		}

		/**
		* Imports the asset from disk
		*/
		void ImportAsset()
		{
			//Default to failed
			State = EState::Done;

			TestReport.AssetName = FPaths::GetCleanFilename(ImportPath);

			//Get the factory
			const FString FileExtension = FPaths::GetExtension(ImportPath);
			UClass* FactoryClass = FAutomationEditorCommonUtils::GetFactoryClassForType(FileExtension);

			if (FactoryClass)
			{
				GWarn->BeginSlowTask(LOCTEXT("ImportSlowTask", "Importing"), true);

				UFactory* ImportFactory = NewObject<UFactory>(GetTransientPackage(), FactoryClass);

				//Apply any custom settings to the factory
				FAutomationEditorCommonUtils::ApplyCustomFactorySettings(ImportFactory, FactorySettings);

				FString Name = ObjectTools::SanitizeObjectName(FPaths::GetBaseFilename(ImportPath));
				FString PackageName = FString::Printf(TEXT("/Game/Automation_Imports/%s"), *Name);

				ImportedAsset = FAutomationEditorCommonUtils::ImportAssetUsingFactory(ImportFactory, Name, PackageName, ImportPath);

				if (ImportedAsset)
				{
					TestReport.bImportSuccessful = true;
					State = EState::OpenEditor;
				}

				GWarn->EndSlowTask();
			}
			else
			{
				UE_LOGF(LogEditorAssetAutomationTests, Error, "Failed to find an import factory for %ls!", *FPaths::GetBaseFilename(ImportPath));
			}
		}

		/**
		* Opens the asset editor
		*/
		void OpenEditor()
		{
			State = EState::Done;

			if (ImportedAsset)
			{
				// Do not show progress window because we need to take screenshot
				// ActiveTopLevelWindow does not get set to editor window for asset if progress modal window was open
				const bool bShowProgressWindow = false;
				if (GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ImportedAsset, EToolkitMode::Standalone, TSharedPtr<IToolkitHost>(), bShowProgressWindow))
				{
					State = EState::WaitForEditor;
				}
				else
				{
					UE_LOGF(LogEditorAssetAutomationTests, Error, "Failed to open the asset editor for %ls!", *ImportedAsset->GetName());
				}
			}
		}

		/**
		* Wait for the asset editor window
		*/
		void CheckEditor()
		{
			TSharedPtr<SWindow> ActiveWindow = FSlateApplication::Get().GetActiveTopLevelWindow();
			if (ActiveWindow.IsValid())
			{
				FString ActiveWindowTitle = ActiveWindow->GetTitle().ToString();

				//Check that we have the right window (Tutorial may have opened on top of the editor)
				if (!ActiveWindowTitle.StartsWith(ImportedAsset->GetName()))
				{
					//Bring the asset editor to the front
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(ImportedAsset, true);
				}

				State = EState::Screenshot;
			}
			else
			{
				WaitingForEditorCount++;
				if (WaitingForEditorCount > MaxWaitForEditorTicks)
				{

					UE_LOGF(LogEditorAssetAutomationTests, Error, "Timed out waiting for editor window: %ls", *ImportedAsset->GetName());
					State = EState::Done;
				}
			}
		}

		/**
		* Take a screenshot of the editor window
		*/
		void TakeScreenshot()
		{
			TSharedPtr<SWindow> ActiveWindow = FSlateApplication::Get().GetActiveTopLevelWindow();
			if (ActiveWindow.IsValid())
			{
				FString ScreenshotPath;
				const FString TestName = FString::Printf(TEXT("AssetImportExport/Screenshots/%s"), *ImportedAsset->GetName());
				ScreenshotPath = AutomationCommon::GetScreenshotPath(TestName);

				TSharedRef<SWidget> WindowRef = ActiveWindow.ToSharedRef();

				TArray<FColor> OutImageData;
				FIntVector OutImageSize;
				if (FSlateApplication::Get().TakeScreenshot(WindowRef, OutImageData, OutImageSize))
				{
					FAutomationScreenshotData Data;
					Data.Width = OutImageSize.X;
					Data.Height = OutImageSize.Y;
					Data.ScreenshotPath = ScreenshotPath;
					FAutomationTestFramework::Get().OnScreenshotCaptured().ExecuteIfBound(OutImageData, Data);
				}

				//Close the editor
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllAssetEditors();

				State = EState::Export;
			}
			else
			{
				UE_LOGF(LogEditorAssetAutomationTests, Error, "No asset editor window found: %ls", *ImportedAsset->GetName());
				State = EState::Done;
			}
		}

		/**
		* Export the asset based of the export extension
		*/
		void ExportAsset()
		{
			if (!bSkipExport)
			{
				FString Extension = ExportExtension;
				if (Extension.Len() == 0)
				{
					//Get the extension from the imported filename
					Extension = FPaths::GetExtension(ImportPath);
				}

				if (Extension.StartsWith(TEXT("."), ESearchCase::CaseSensitive))
				{
					Extension.RightChopInline(1, EAllowShrinking::No);
				}

				//Export the asset
				const FString ExportAssetName = FString::Printf(TEXT("%s.%s"), *ImportedAsset->GetName(), *Extension);

				UAssetExportTask* ExportTask = NewObject<UAssetExportTask>();
				FGCObjectScopeGuard ExportTaskGuard(ExportTask);
				ExportTask->Object = ImportedAsset;
				ExportTask->Exporter = UExporter::FindExporter(ImportedAsset, *Extension);
				ExportTask->Filename = FPaths::Combine(*FPaths::AutomationDir(), TEXT("AssetImportExport"), TEXT("Exported"), *ExportAssetName);
				ExportTask->bSelected = false;
				ExportTask->bReplaceIdentical = true;
				ExportTask->bPrompt = false;
				ExportTask->bUseFileArchive = ImportedAsset->IsA(UPackage::StaticClass());
				ExportTask->bWriteEmptyFiles = false;
				ExportTask->bAutomated = true;
				if (UExporter::RunAssetExportTask(ExportTask) != 0)  //1 - success, 0 - fatal error, -1 - non fatal error
				{
					//Success
					TestReport.bExportSuccessful = true;
					TestReport.FileSize = IFileManager::Get().FileSize(*ExportTask->Filename);
				}
				else
				{
					UE_LOGF(LogEditorAssetAutomationTests, Error, "Failed to export asset: %ls", *ImportedAsset->GetName());
				}
			}
			else
			{
				TestReport.bSkippedExport = true;
			}

			State = EState::Done;
		}

		/**
		* Delete the assset
		*/
		void DeleteAsset()
		{
			// Deselect all
			GEditor->SelectNone(false, true);

			// Clear the transaction buffer so we aren't referencing the new objects
			GUnrealEd->ResetTransaction(FText::FromString(TEXT("FAssetEditorTest")));

			//Clear references to the object so we can delete it
			FAutomationEditorCommonUtils::NullReferencesToObject(ImportedAsset);

			//Delete the object
			TArray<UObject*> ObjList;
			ObjList.Add(ImportedAsset);
			ObjectTools::ForceDeleteObjects(ObjList, false);

			ImportedAsset = NULL;
		}
	};
}

/**
* Latent command to update the asset helper
*/
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FImportExportAssetCommand, TSharedPtr<ImportExportAssetHelper::FAssetInfo>, AssetHelper);
bool FImportExportAssetCommand::Update()
{
	return AssetHelper->Update();
}

/**
* Latent command to update the asset helper
*/
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FLogImportExportTestResultsCommand, TSharedPtr<ImportExportAssetHelper::FAssetImportStats>, BuildStats);
bool FLogImportExportTestResultsCommand::Update()
{
	UE_LOGF(LogEditorAssetAutomationTests, Display, " ");
	UE_LOGF(LogEditorAssetAutomationTests, Display, "Test Summary:");
	for (int32 i = 0; i < BuildStats->Reports.Num(); ++i)
	{
		UE_LOGF(LogEditorAssetAutomationTests, Display, "AssetName: %ls", *BuildStats->Reports[i].AssetName);
		UE_LOGF(LogEditorAssetAutomationTests, Display, "  Import: %ls", BuildStats->Reports[i].bImportSuccessful ? TEXT("SUCCESS") : TEXT("FAILED"));
		if (BuildStats->Reports[i].bSkippedExport)
		{
			UE_LOGF(LogEditorAssetAutomationTests, Display, "  Export: SKIPPED");
		}
		else if (BuildStats->Reports[i].bExportSuccessful)
		{
			UE_LOGF(LogEditorAssetAutomationTests, Display, "  Export: SUCCESS FileSize: %i", BuildStats->Reports[i].FileSize);
		}
		else
		{
			UE_LOGF(LogEditorAssetAutomationTests, Display, "  Export: FAILED");
		}
	}
	return true;
}

/**
* Automation test to import, open, screenshot, and export assets
*/
IMPLEMENT_COMPLEX_AUTOMATION_TEST(FAssetImportEditorTest, "Project.Editor.Content.Asset Import and Export", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
void FAssetImportEditorTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
	check(AutomationTestSettings);
	
	for ( int32 TestIdx = 0; TestIdx < AutomationTestSettings->ImportExportTestDefinitions.Num(); ++TestIdx )
	{
		// Lets just use the filename with no path, and change .'s to _'s so the parser
		// doesn't get confused and insert a bunch of children tests from all the .'s in the path.
		// TODO Introduce a way to escape .'s
		FString CleanFileName = FPaths::GetCleanFilename(AutomationTestSettings->ImportExportTestDefinitions[TestIdx].ImportFilePath.FilePath);
		CleanFileName = CleanFileName.Replace(TEXT("."), TEXT("_"));

		OutBeautifiedNames.Add(CleanFileName);
		OutTestCommands.Add(FString::FromInt(TestIdx));
	}
}

bool FAssetImportEditorTest::RunTest(const FString& Parameters)
{
	UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
	check(AutomationTestSettings);

	TSharedPtr<ImportExportAssetHelper::FAssetImportStats> BuildStats = MakeShareable(new ImportExportAssetHelper::FAssetImportStats());

	int32 TestIdx = FCString::Atoi(*Parameters);
	
	TSharedPtr<ImportExportAssetHelper::FAssetInfo> AssetInfo = MakeShareable(new ImportExportAssetHelper::FAssetInfo(AutomationTestSettings->ImportExportTestDefinitions[TestIdx], &ExecutionInfo, BuildStats));
	ADD_LATENT_AUTOMATION_COMMAND(FImportExportAssetCommand(AssetInfo));

	ADD_LATENT_AUTOMATION_COMMAND(FLogImportExportTestResultsCommand(BuildStats));

	return true;
}

#undef LOCTEXT_NAMESPACE
