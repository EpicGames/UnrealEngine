// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "HAL/IConsoleManager.h"
#include "Misc/AutomationTest.h"
#include "Modules/ModuleManager.h"
#include "UObject/Object.h"
#include "UObject/GarbageCollection.h"
#include "UObject/UObjectIterator.h"
#include "Misc/PackageName.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/Blueprint.h"
#include "Materials/MaterialInterface.h"
#include "Engine/SkeletalMesh.h"
#include "Components/PrimitiveComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "Engine/Brush.h"
#include "Engine/BlockingVolume.h"
#include "Engine/DirectionalLight.h"
#include "Engine/PointLight.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/Selection.h"
#include "EngineUtils.h"
#include "Builders/CubeBuilder.h"
#include "LevelEditorViewport.h"
#include "EditorModeManager.h"
#include "FileHelpers.h"

#include "Tests/AutomationTestSettings.h"
#include "AssetSelection.h"
#include "ScopedTransaction.h"
#include "Interfaces/IMainFrameModule.h"
#include "EngineAnalytics.h"
#include "Interfaces/IAnalyticsProvider.h"

#include "EdGraphSchema_K2.h"
#include "K2Node.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include "Tests/AutomationCommon.h"
#include "Tests/AutomationEditorCommon.h"

#include "PackageTools.h"
#include "Editor/Transactor.h"

#include "K2Node_Event.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_Timeline.h"
#include "K2Node_Tunnel.h"
#include "K2Node_MacroInstance.h"
#include "K2Node_CallFunction.h"
#include "K2Node_Composite.h"
#include "Subsystems/EditorActorSubsystem.h"

/**
* Change the attributes for a point light in the level.
*/
struct PointLightParameters
{
	APointLight* PointLight;
	float LightBrightness;
	float LightRadius;
	FVector LightLocation;
	FColor LightColor;
	PointLightParameters()
		: PointLight(nullptr)
		, LightBrightness(5000.0f)
		, LightRadius(1000.0f)
		, LightLocation(FVector(0.0f, 0.0f, 0.0f))
		, LightColor(FColor::White)
	{
	}
};

//Updates the properties of a specified point light.
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(PointLightUpdateCommand, PointLightParameters, PointLightUsing);

bool PointLightUpdateCommand::Update()
{
	//Set the point light mobility, brightness, radius, and light color.
	PointLightUsing.PointLight->SetMobility(EComponentMobility::Movable);
	PointLightUsing.PointLight->SetBrightness(PointLightUsing.LightBrightness);
	PointLightUsing.PointLight->SetLightColor(PointLightUsing.LightColor);
	PointLightUsing.PointLight->TeleportTo(PointLightUsing.LightLocation, FRotator(0, 0, 0));
	PointLightUsing.PointLight->SetRadius(PointLightUsing.LightRadius);
	return true;
}

/**
* Duplicates a point light.
*/
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(PointLightDuplicationCommand, PointLightParameters, PointLightDuplicating);

bool PointLightDuplicationCommand::Update()
{

	FScopedTransaction DuplicateLightScope(NSLOCTEXT("UnrealEd.Test", "DuplicateLightScope", "Duplicate Light Scope"));

	//Duplicate the light.
	bool bOffsetLocations = false;
	GEditor->edactDuplicateSelected(PointLightDuplicating.PointLight->GetLevel(), bOffsetLocations);
	for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
	{
		AActor* Actor = static_cast<AActor*>(*It);
		Actor->TeleportTo(FVector(PointLightDuplicating.LightLocation), FRotator(0, 0, 0));
	}
	return true;
}

//////////////////////////////////////////////////////////////////////////

/**
 * FGenericImportAssetsAutomationTest
 * Simple unit test that attempts to import every file (except .txt files) within the unit test directory in a sub-folder
 * named "GenericImport." Used to test the basic codepath that would execute if a user imported a file using the interface
 * in the Content Browser (does not allow for specific settings to be made per import factory). Cannot be run in a commandlet
 * as it executes code that routes through Slate UI.
 */
IMPLEMENT_COMPLEX_AUTOMATION_TEST(FGenericImportAssetsAutomationTest, "Editor.Import", (EAutomationTestFlags::EditorContext | EAutomationTestFlags::NonNullRHI | EAutomationTestFlags::EngineFilter))

/** 
 * Requests a enumeration of all sample assets to import
 */
void FGenericImportAssetsAutomationTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const
{
	FString ImportTestDirectory;
	check( GConfig );
	GConfig->GetString( TEXT("AutomationTesting"), TEXT("ImportTestPath"), ImportTestDirectory, GEngineIni );
	ImportTestDirectory = FPaths::Combine(FPaths::ConvertRelativePathToFull(FPaths::EngineDir()), ImportTestDirectory);

	// Find all files in the GenericImport directory
	TArray<FString> FilesInDirectory;
	IFileManager::Get().FindFilesRecursive(FilesInDirectory, *ImportTestDirectory, TEXT("*.*"), true, false);
	
	// Scan all the found files, ignoring .txt files which are likely P4 placeholders for creating directories
	for ( TArray<FString>::TConstIterator FileIter( FilesInDirectory ); FileIter; ++FileIter )
	{
		FString Filename( *FileIter );
		FString Ext = FPaths::GetExtension(Filename, true);
		if ( Ext != TEXT(".txt") && !FPackageName::IsPackageExtension(*Ext) )
		{
			FString FileString = *FileIter;
			OutBeautifiedNames.Add(FPaths::GetBaseFilename(Filename));
			OutTestCommands.Add(FileString);
		}
	}
}

/** 
 * Execute the generic import test
 *
 * @param Parameters - Should specify the asset to import
 * @return	TRUE if the test was successful, FALSE otherwise
 */
bool FGenericImportAssetsAutomationTest::RunTest(const FString& Parameters)
{
	TArray<FString> CurFileToImport;
	CurFileToImport.Add( *Parameters );
	FString CleanFilename = FPaths::GetCleanFilename(CurFileToImport[0]);

	FString PackagePath;
	check( GConfig );
	GConfig->GetString( TEXT("AutomationTesting"), TEXT("ImportTestPackagePath"), PackagePath, GEngineIni );

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	PushContext(CleanFilename);
	TArray<UObject*> ImportedObjects = AssetToolsModule.Get().ImportAssets(CurFileToImport, PackagePath);
	PopContext();
	
	return ImportedObjects.Num() == 1;
}

/**
 * LoadAllMaps
 * Verification automation test to make sure loading all maps succeed without crashing
 */
IMPLEMENT_COMPLEX_AUTOMATION_TEST(FLoadAllMapsInEditorTest, "Project.Maps.Load All In Editor", EAutomationTestFlags::EditorContext | EAutomationTestFlags::StressFilter)

/** 
 * Requests a enumeration of all maps to be loaded
 */
void FLoadAllMapsInEditorTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const
{
	TArray<FString> FileList;
	FEditorFileUtils::FindAllPackageFiles(FileList);

	// Iterate over all files, adding the ones with the map extension..
	for( int32 FileIndex=0; FileIndex< FileList.Num(); FileIndex++ )
	{
		const FString& Filename = FileList[FileIndex];

		// Disregard filenames that don't have the map extension if we're in MAPSONLY mode.
		if ( FPaths::GetExtension(Filename, true) == FPackageName::GetMapPackageExtension()) 
		{
			if (FAutomationTestFramework::Get().ShouldTestContent(Filename))
			{
				if (!Filename.Contains(TEXT("/Engine/")))
				{
					OutBeautifiedNames.Add(FPaths::GetBaseFilename(Filename));
					OutTestCommands.Add(Filename);
				}
			}
		}
	}
}


/** 
 * Execute the loading of each map
 *
 * @param Parameters - Should specify which map name to load
 * @return	TRUE if the test was successful, FALSE otherwise
 */
bool FLoadAllMapsInEditorTest::RunTest(const FString& Parameters)
{
	FString MapName = Parameters;
	double MapLoadStartTime = 0;

	//Test event for analytics. This should fire anytime this automation procedure is started.
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent( TEXT("Editor.Usage.TestEvent"));
		UE_LOGF(LogEditorAutomationTests, Log, "AnayticsTest: Load All Maps automation triggered and Editor.Usage.TestEvent analytic event has been fired.");
	}
	

	{
		//Find the main editor window
		TArray<TSharedRef<SWindow> > AllWindows;
		FSlateApplication::Get().GetAllVisibleWindowsOrdered(AllWindows);
		if( AllWindows.Num() == 0 )
		{
			UE_LOGF(LogEditorAutomationTests, Error, "ERROR: Could not find the main editor window.");
			return false;
		}
		WindowScreenshotParameters WindowParameters;
		WindowParameters.CurrentWindow = AllWindows[0];

		//Disable Eye Adaptation
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.EyeAdaptationQuality"));
		CVar->Set(0);

		//Create a screen shot filename and path
		const FString LoadAllMapsTestName = FString::Printf(TEXT("LoadAllMaps_Editor/%s"), *FPaths::GetBaseFilename(MapName));
		WindowParameters.ScreenshotName = AutomationCommon::GetScreenshotPath(LoadAllMapsTestName);

		//Get the current number of seconds.  This will be used to track how long it took to load the map.
		MapLoadStartTime = FPlatformTime::Seconds();
		//Load the map
		FAutomationEditorCommonUtils::LoadMap(MapName);
		//Log how long it took to launch the map.
		UE_LOGF(LogEditorAutomationTests, Display, "Map '%ls' took %.3f to load", *MapName, FPlatformTime::Seconds() - MapLoadStartTime);


		//If we don't have NoTextureStreaming enabled, give the textures some time to load.
		if( !FParse::Param( FCommandLine::Get(), TEXT( "NoTextureStreaming" ) ) )
		{
			//Give the contents some time to load
			ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(1.5f));
		}
	}

	return true;
}

//////////////////////////////////////////////////////////////////////////

/**
 * Reinitialize all RHI resources
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FReinitializeRHIResources, "System.Engine.Rendering.Reinit Resources", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FReinitializeRHIResources::RunTest(const FString& Parameters)
{
	GEditor->Exec( NULL, TEXT("ReinitRHIResources"));
	return true;
}


//////////////////////////////////////////////////////////////////////////
/**
 * QA Static Mesh Regression Testing
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStaticMeshValidation, "System.QA.Mesh Factory Validation", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FStaticMeshValidation::RunTest(const FString& Parameters)
{
	UWorld* World = FAutomationEditorCommonUtils::CreateNewMap();

	//Adjust camera in viewports
	for(FLevelEditorViewportClient* ViewportClient : GEditor->GetLevelViewportClients())
	{
		if(!ViewportClient->IsOrtho())
		{
			ViewportClient->SetViewLocation( FVector(67, 1169, 1130) );
			ViewportClient->SetViewRotation( FRotator(321, 271, 0) );
		}
	}

	//Gather assets
	UObject* EditorCubeMesh = (UStaticMesh*)StaticLoadObject(UStaticMesh::StaticClass(),NULL,TEXT("/Engine/EditorMeshes/EditorCube.EditorCube"),NULL,LOAD_None,NULL);
	UObject* EditorSkeletalMesh = (USkeletalMesh*)StaticLoadObject(USkeletalMesh::StaticClass(),NULL,TEXT("/Engine/EditorMeshes/SkeletalMesh/DefaultSkeletalMesh.DefaultSkeletalMesh"),NULL,LOAD_None,NULL);

	// Static Mesh 0
	AActor* StaticMesh = FActorFactoryAssetProxy::AddActorForAsset( EditorCubeMesh );
	StaticMesh->TeleportTo(FVector(0.0f, 0.0f, 0.0f), FRotator(0, 0, 0));
	StaticMesh->SetActorRelativeScale3D(FVector(50.0f, 50.0f, 1.0f));

	//Static Mesh 1
	StaticMesh = FActorFactoryAssetProxy::AddActorForAsset( EditorCubeMesh );
	StaticMesh->TeleportTo(FVector(-816.0f, -512.0f, 382.0f), FRotator(64, -64, 32));
	StaticMesh->SetActorRelativeScale3D(FVector(1.0f, 1.0f, 2.0f));

	//Interp Actor
	AActor* InterpActor = FActorFactoryAssetProxy::AddActorForAsset( EditorCubeMesh );
	InterpActor->GetRootComponent()->SetMobility(EComponentMobility::Movable);
	{
		bool bIsATest = false;
		bool bNoCheck = true;
		InterpActor->TeleportTo(FVector(-900.0f, 196.0f, 256.0f), FRotator(0, 0, 0), bIsATest, bNoCheck);
	}

	//Physics Actor
	AActor* PhysicsActor = FActorFactoryAssetProxy::AddActorForAsset( EditorCubeMesh );
	PhysicsActor->SetActorRelativeScale3D(FVector(2.0f, 2.0f, .5f));
	PhysicsActor->GetRootComponent()->SetMobility(EComponentMobility::Movable);
	CastChecked<UPrimitiveComponent>(PhysicsActor->GetRootComponent())->SetSimulatePhysics(true);
	PhysicsActor->TeleportTo(FVector(-96.0f, 128.0f, 256.0f), FRotator(0, 0, 0));

	//Skeletal Mesh
	AActor* SkeletalMesh = FActorFactoryAssetProxy::AddActorForAsset( EditorSkeletalMesh );
	SkeletalMesh->SetActorLocationAndRotation(FVector(640.0f, 196.0f, 256.0f), FRotator(12, .5, 24));
	SkeletalMesh->SetActorRelativeScale3D(FVector(2.0f, 3.0f, 2.5f));

	//Single Anim Skeletal Mesh
	//UActorFactory* SingleAnimSkeletalActorFactory = GEditor->FindActorFactoryForActorClass( ASingleAnimSkeletalActor::StaticClass() );
	//SkeletalMesh = FActorFactoryAssetProxy::AddActorForAsset( EditorSkeletalMesh, false, true, RF_Transactional, SingleAnimSkeletalActorFactory );
	//SkeletalMesh->TeleportTo(FVector(1152.0f, 256.0f, 256.0f), FRotator(0, 0, 0));

	//Directional Light
	const FTransform Transform(FVector(-611.0f, 242.0f, 805.0f));
	ADirectionalLight* DirectionalLight = Cast<ADirectionalLight>(GEditor->AddActor(World->GetCurrentLevel(), ADirectionalLight::StaticClass(), Transform));
	DirectionalLight->SetMobility(EComponentMobility::Movable);
	DirectionalLight->SetActorRotation(FRotator(329, 346, -105));
	DirectionalLight->SetBrightness(3.142f);
	DirectionalLight->SetLightColor(FColor::White);

	GLevelEditorModeTools().MapChangeNotify();

	return true;
}



//////////////////////////////////////////////////////////////////////////
/**
 * QA Convert Meshes Regression Testing
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FConvertToValidation, "System.QA.Convert Meshes", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

//gather all brushes that exist now
void ConvertTestFindAllBrushes(TArray<ABrush*> &PreviousBrushes)
{
	for( TObjectIterator<ABrush> It; It; ++It )
	{
		ABrush* BrushActor = *It;
		PreviousBrushes.Add(BrushActor);
	}
}

//find brush that was just added by finding the brush not in our previous list
ABrush* ConvertTestFindNewBrush (const TArray<ABrush*> &PreviousBrushes)
{
	ABrush* NewBrush = NULL;
	for( TObjectIterator<ABrush> It; It; ++It )
	{
		ABrush* BrushActor = *It;
		if (!PreviousBrushes.Contains(BrushActor))
		{
			NewBrush = BrushActor;
			break;
		}
	}
	check( NewBrush );
	return NewBrush;
}

/**
 * Parameters to the Latent Automation command FCleanupConvertToValidation
 */
struct FCleanupConvertToValidationParameters
{
	TWeakObjectPtr<UWorld> TestWorld;
	FString AssetPackageName;
};

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FCleanupConvertToValidation, FCleanupConvertToValidationParameters, Parameters);

bool FConvertToValidation::RunTest(const FString& Parameters)
{
	UWorld* World = FAutomationEditorCommonUtils::CreateNewMap();

	//Set the Test Name which is used later for getting the directory to store the screenshots.
	const FString BaseFileName = TEXT("ConvertMeshTest");
	
	//Creating the parameters needed for latent screenshot capturing.
	WindowScreenshotParameters ConvertMeshParameters;
	
	//Check if the main frame is loaded.  When using the old main frame it may not be.
	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		//Now set the WindowScreenshot struct CurrentWindow name to be the mainframe.
		ConvertMeshParameters.CurrentWindow = MainFrame.GetParentWindow();
	}

	//Set the screenshot name.
	ConvertMeshParameters.ScreenshotName = BaseFileName;

	//Adjust camera in viewports
	for(FLevelEditorViewportClient* ViewportClient : GEditor->GetLevelViewportClients())
	{
		if(!ViewportClient->IsOrtho())
		{
			ViewportClient->SetViewLocation( FVector(190, 590, 360) );
			ViewportClient->SetViewRotation( FRotator(0, -90, 0) );
		}
	}

	//BSP TO BLOCKING VOLUME
	{
		// Note: Rebuilding BSP requires a transaction.
		FScopedTransaction Transaction(NSLOCTEXT("EditorAutomation", "ConvertBSPToBlocking", "Convert BSP to Blocking Volume"));

		TArray<ABrush*> PreviousBrushes;
		ConvertTestFindAllBrushes(PreviousBrushes);

		//Add the new brush
		UCubeBuilder* CubeAdditiveBrushBuilder = Cast<UCubeBuilder>(GEditor->FindBrushBuilder( UCubeBuilder::StaticClass() ));
		CubeAdditiveBrushBuilder->X = 256.0f;
		CubeAdditiveBrushBuilder->Y = 256.0f;
		CubeAdditiveBrushBuilder->Z = 256.0f;
		CubeAdditiveBrushBuilder->Build(World);
		GEditor->Exec( World, TEXT("BRUSH MOVETO X=384 Y=0 Z=384"));
		GEditor->Exec( World, TEXT("BRUSH ADD"));

		//find brush that was just added by finding the brush not in our previous list
		ABrush* NewBrush = ConvertTestFindNewBrush(PreviousBrushes);
		check( NewBrush );

		//modify selection - convert to blocking volume
		const bool bNoteSelectionChange = true;
		const bool bDeselectBSPSurfaces = true;
		GEditor->SelectNone(bNoteSelectionChange, bDeselectBSPSurfaces);
		GEditor->GetSelectedActors()->Modify();
		GEditor->GetSelectedActors()->Select( NewBrush );	
		UEditorActorSubsystem::ConvertSelectedBrushesToVolumes( ABlockingVolume::StaticClass() );
		GEditor->RebuildAlteredBSP();

		// During automation we do not actually care about creating a transaction for the user to undo.  
		Transaction.Cancel();
	}

	//convert to static mesh
	FString AssetPackageName;
	{
		TArray<ABrush*> PreviousBrushes;
		ConvertTestFindAllBrushes(PreviousBrushes);

		//Add the new brush
		UCubeBuilder* CubeAdditiveBrushBuilder = Cast<UCubeBuilder>(GEditor->FindBrushBuilder( UCubeBuilder::StaticClass() ));
		CubeAdditiveBrushBuilder->X = 256.0f;
		CubeAdditiveBrushBuilder->Y = 256.0f;
		CubeAdditiveBrushBuilder->Z = 256.0f;
		CubeAdditiveBrushBuilder->Build(World);
		GEditor->Exec( World, TEXT("BRUSH MOVETO X=0 Y=0 Z=384"));
		GEditor->Exec( World, TEXT("BRUSH ADD"));

		//find brush that was just added by finding the brush not in our previous list
		ABrush* NewBrush = ConvertTestFindNewBrush(PreviousBrushes);
		check( NewBrush );
		TArray<AActor*> ToStaticMeshActors;
		ToStaticMeshActors.Add(NewBrush);

		//generate static mesh package name. Temporarily mount /Automation.
		FPackageName::RegisterMountPoint(TEXT("/Automation/"), FPaths::AutomationTransientDir());
		AssetPackageName = TEXT("/Automation/ConvertToBSPToStaticMesh");
		//Convert brush to specific package name
		UEditorActorSubsystem::DoConvertActors(ToStaticMeshActors, AStaticMeshActor::StaticClass(), TSet<FString>(), true, AssetPackageName);

		//find the package
		UPackage* NewPackage = FindPackage(NULL, *AssetPackageName);
		if (NewPackage)
		{
			TArray<UPackage*> PackagesToSave;
			PackagesToSave.Add(NewPackage);

			//save the package
			const bool bCheckDirty = false;
			const bool bPromptToSave = false;
			const FEditorFileUtils::EPromptReturnCode Return = FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirty, bPromptToSave);
		}
		else
		{
			UE_LOGF(LogEditorAutomationTests, Error, "Failed to save ConvertToBSPToStaticMesh.");
		}
	}
	
	//Wait to give the screenshot capture some time to complete.
	ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(0.1f));

	//Directional Light
	const FTransform Transform(FVector(384, 0, 384));
	ADirectionalLight* DirectionalLight = Cast<ADirectionalLight>(GEditor->AddActor(World->GetCurrentLevel(), ADirectionalLight::StaticClass(), Transform));
	DirectionalLight->SetMobility(EComponentMobility::Movable);
	DirectionalLight->SetActorRotation(FRotator(314, 339, 0));
	DirectionalLight->SetBrightness(3.142f);
	DirectionalLight->SetLightColor(FColor::White);

	GLevelEditorModeTools().MapChangeNotify();

	// Add a latent action to clean up the static mesh actor we created and unload the temporary asset AFTER we take the screenshot
	FCleanupConvertToValidationParameters CleanupParameters;
	CleanupParameters.AssetPackageName = AssetPackageName;
	CleanupParameters.TestWorld = World;
	ADD_LATENT_AUTOMATION_COMMAND(FCleanupConvertToValidation(CleanupParameters));

	return true;
}

bool FCleanupConvertToValidation::Update()
{
	const FString& AssetPackageName = Parameters.AssetPackageName;
	UWorld* TestWorld = Parameters.TestWorld.Get();

	// Attempt to unload the asset we created temporarily.
	UPackage* NewPackage = FindPackage(NULL, *AssetPackageName);
	if (NewPackage)
	{
		if (TestWorld)
		{
			// First find the static mesh we made in this package
			UStaticMesh* GeneratedMesh = FindObject<UStaticMesh>(NewPackage, *FPackageName::GetLongPackageAssetName(AssetPackageName));

			// If we found the mesh, find and delete the static mesh actor we added to the level to clear the reference to it.
			if (GeneratedMesh)
			{
				for (TActorIterator<AStaticMeshActor> ActorIt(TestWorld); ActorIt; ++ActorIt)
				{
					AStaticMeshActor* StaticMeshActor = *ActorIt;

					if ( StaticMeshActor->GetStaticMeshComponent()->GetStaticMesh() == GeneratedMesh )
					{
						TestWorld->DestroyActor(StaticMeshActor);
					}
				}
			}
		}

		// Clear the transaction buffer to remove the last reference
		GEditor->Trans->Reset( NSLOCTEXT( "UnrealEd.Test", "ConvertToValidationClear", "ConvertToValidation Clear" ) );

		// Now unload the package
		TArray<UPackage*> PackagesToUnload;
		PackagesToUnload.Add(NewPackage);
		UPackageTools::UnloadPackages(PackagesToUnload);
	}

	// Unmount /Automation.
	FPackageName::UnRegisterMountPoint(TEXT("/Automation/"), FPaths::AutomationTransientDir());

	return true;
}


//////////////////////////////////////////////////////////////////////////
/**
 * QA Static Mesh Regression Testing
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStaticMeshPlacement, "System.QA.Static Mesh Placement", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FStaticMeshPlacement::RunTest(const FString& Parameters)
{
	FString MapName = TEXT("/Engine/Maps/Templates/Template_Default");
	FAutomationEditorCommonUtils::LoadMap(MapName);

	//Gather assets
	UObject* EditorCylinderMesh = (UStaticMesh*)StaticLoadObject(UStaticMesh::StaticClass(),NULL,TEXT("/Engine/EditorMeshes/EditorCylinder.EditorCylinder"),NULL,LOAD_None,NULL);

	// Add cylinder to world
	AStaticMeshActor* StaticMesh = Cast<AStaticMeshActor>(FActorFactoryAssetProxy::AddActorForAsset(EditorCylinderMesh));
	StaticMesh->TeleportTo(FVector(-16.0f, 448.0f, 608.0f), FRotator(0, 0, 0));
	StaticMesh->SetActorRelativeScale3D(FVector(1.0f, 1.0f, 1.0f));

	{
		FScopedTransaction DuplicateMeshScope( NSLOCTEXT("UnrealEd.Test", "UndoStaticMeshPlacementTest", "Undo Static Mesh Placement Test") );

		bool bOffsetLocations = false;
		//Duplicate the mesh
		GEditor->edactDuplicateSelected(StaticMesh->GetLevel(), bOffsetLocations);
		for ( FSelectionIterator It( GEditor->GetSelectedActorIterator() ) ; It ; ++It )
		{
			AActor* Actor = static_cast<AActor*>( *It );
			Actor->TeleportTo(FVector(304.0f,448.0f,608.0f), FRotator(0, 0, 0));
		}
	}

	GEditor->UndoTransaction();

	FString MaterialName = TEXT("/Engine/MapTemplates/Materials/BasicAsset01.BasicAsset01");
	UMaterialInterface* Material = (UMaterialInterface*)StaticLoadObject(UMaterialInterface::StaticClass(),NULL,*MaterialName,NULL,LOAD_None,NULL);
	if (Material)
	{
		FActorFactoryAssetProxy::ApplyMaterialToActor( StaticMesh, Material );
	}
	else
	{
		UE_LOGF(LogEditorAutomationTests, Error, "Failed to find material: %ls", *MaterialName);
	}

	StaticMesh->TeleportTo(FVector(160.0f,448.0f, 608.0f), FRotator(0, 280, 0));
	StaticMesh->SetActorRelativeScale3D(FVector(1.5f, 1.5f, 1.5f));

	return true;
}




//////////////////////////////////////////////////////////////////////////
/**
 * QA Light Placement Regression Testing
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLightPlacement, "System.QA.Point Light Placement", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FLightPlacement::RunTest(const FString& Parameters)
{
	//Initialize the parameters for taking a screenshot as well as update the placed point light.
	WindowScreenshotParameters PointLightPlacementWindowParameters;
	PointLightParameters LightParameters;
	bool bUndo = true;

	//Set the CurrentWindow to the Mainframe.  This information is used for taking a screenshot later.
	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		PointLightPlacementWindowParameters.CurrentWindow = MainFrame.GetParentWindow();
	}

	//Set the Test Name which is used later for getting the directory to store the screenshots.
	const FString BaseFileName = TEXT("PointLightPlacementTest");

	//Open a new blank map.
	UWorld* World = FAutomationEditorCommonUtils::CreateNewMap();

	//Move the perspective viewport view to show the test.
	for(FLevelEditorViewportClient* ViewportClient : GEditor->GetLevelViewportClients())
	{
		if (!ViewportClient->IsOrtho())
		{
			ViewportClient->SetViewLocation(FVector(890, 70, 280));
			ViewportClient->SetViewRotation(FRotator(0, 180, 0));
		}
	}

	//Gather assets.
	UObject* Cube = (UStaticMesh*)StaticLoadObject(UStaticMesh::StaticClass(), NULL, TEXT("/Engine/EngineMeshes/Cube.Cube"), NULL, LOAD_None, NULL);
	//Add Cube mesh to the world
	AStaticMeshActor* StaticMesh = Cast<AStaticMeshActor>(FActorFactoryAssetProxy::AddActorForAsset(Cube));
	StaticMesh->TeleportTo(FVector(0.0f, 0.0f, 0.0f), FRotator(0, 0, 0));
	StaticMesh->SetActorRelativeScale3D(FVector(3.0f, 3.0f, 1.75f));

	//Create the point light and set it's mobility, brightness, and light color.
	const FTransform Transform(FVector(0.0f, 0.0f, 400.0f));
	APointLight* PointLight = Cast<APointLight>(GEditor->AddActor(World->GetCurrentLevel(), APointLight::StaticClass(), Transform));
	LightParameters.PointLight = PointLight;
	LightParameters.LightColor = FColor::Red;
	LightParameters.LightLocation = FVector(0.0f, 0.0f, 400.0f);
	ADD_LATENT_AUTOMATION_COMMAND(PointLightUpdateCommand(LightParameters));

	//Wait
	ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(0.1f));

	//Duplicate the point light.
	LightParameters.LightLocation = FVector(10.0f, 10.0f, 400.0f);
	ADD_LATENT_AUTOMATION_COMMAND(PointLightDuplicationCommand(LightParameters));

	//Undo the duplication.
	ADD_LATENT_AUTOMATION_COMMAND(FUndoRedoCommand(true));

	//Redo the duplication.
	ADD_LATENT_AUTOMATION_COMMAND(FUndoRedoCommand(false));

	//Update the original point light actor.
	LightParameters.LightRadius = 500.0f;
	LightParameters.LightLocation = FVector(500.0f, 300.0f, 500.0f);
	LightParameters.LightColor = FColor::White;
	ADD_LATENT_AUTOMATION_COMMAND(PointLightUpdateCommand(LightParameters));

	//Wait
	ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(0.1f));

	return true;
}



/**
 * TraceAllTimelines
 * Unit test to find all timelines in blueprints and list the events that can trigger them.
 * Timelines implicitly tick and are usually used for cosmetic events, so they can cause performance problems on dedicated servers.
 */
IMPLEMENT_COMPLEX_AUTOMATION_TEST(FTraceAllTimelinesAutomationTest, "Project.Performance Audits.Find Timelines On Server", (EAutomationTestFlags::EditorContext | EAutomationTestFlags::StressFilter))

/** 
 * Requests an enumeration of all blueprints to be loaded
 */
void FTraceAllTimelinesAutomationTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const
{
	FAutomationEditorCommonUtils::CollectTestsByClass(UBlueprint::StaticClass(), OutBeautifiedNames, OutTestCommands);
}


namespace TraceHelper
{
	static const FName CosmeticMacroName(TEXT("Can Execute Cosmetic Events"));
	static const FName CosmeticCheckedPinName(TEXT("True"));
	static const FName SetTimerName(TEXT("K2_SetTimer"));
	static const FName SetTimerFunctionFieldName(TEXT("FunctionName"));

	typedef TArray<UK2Node_MacroInstance*, TInlineAllocator<2>> TContextStack;
	typedef TMap<UK2Node_MacroInstance*, struct FVisitedTracker> TGraphNodesVisited;
	typedef TArray<struct FNodeScope, TInlineAllocator<32>> TNodeScopeStack;
	typedef TSet<struct FVisitedNode> TVisitedNodeSet;

	// converts a bool indicating whether we are looking at cosmetic or non-cosmetic nodes into an index in an array.
	inline int32 ToIndex(bool bCosmeticChain)
	{
		return bCosmeticChain ? 0 : 1;
	}

	// Node we have visited (and pin we visited through)
	struct FVisitedNode
	{
		FVisitedNode(UK2Node* InNode = NULL, const UEdGraphPin* InPin = NULL)
			: Node(InNode)
			, Pin(InPin)
		{
		}

		bool operator==(const FVisitedNode& Other) const
		{
			return Node == Other.Node && Pin == Other.Pin;
		}

		UK2Node* Node;
		const UEdGraphPin* Pin;
	};

	inline uint32 GetTypeHash( const struct FVisitedNode& VisitedNode )
	{
		const UPTRINT NodePtr = reinterpret_cast<UPTRINT>(VisitedNode.Node);
		const UPTRINT PinPtr = reinterpret_cast<UPTRINT>(VisitedNode.Pin);
		return ::GetTypeHash(reinterpret_cast<void*>(NodePtr ^ PinPtr));
	}

	// Track nodes that have been visited within a context, either along a cosmetic execution chain or non-cosmetic (checked).
	// This is necessary to avoid infinite loops when tracing a sequence in a graph.
	struct FVisitedTracker
	{
		bool IsVisited(UK2Node* Node, bool bCosmeticChain, const UEdGraphPin* Pin) const
		{
			const int32 Index = ToIndex(bCosmeticChain);
			return Nodes[Index].Contains(FVisitedNode(Node, Pin));
		}

		void AddNode(UK2Node* Node, bool bCosmeticChain, const UEdGraphPin* Pin)
		{
			const int32 Index = ToIndex(bCosmeticChain);
			Nodes[Index].Add(FVisitedNode(Node, Pin));
		}

		TVisitedNodeSet Nodes[2];
	};

	// K2Node and macro context stack within which the node exists
	struct FNodeScope
	{
		FNodeScope(UK2Node* InNode, const TContextStack& InContextStack, bool bInCosmeticChain, const UEdGraphPin* InPin)
			: Node(InNode)
			, Pin(InPin)
			, ContextStack(InContextStack)
			, bCosmeticChain(bInCosmeticChain)
		{
			check(InNode != NULL);
		}

		UK2Node*			Node;
		const UEdGraphPin*	Pin; // If NULL, then consider all pins for this node.
		TContextStack		ContextStack;
		bool				bCosmeticChain;
	};

	// Add a node to the NodeScopeStack if it has not been visited already.
	bool AddNode(UK2Node* Node, TNodeScopeStack& NodeScopeStack, TGraphNodesVisited& GraphNodesVisited, const FNodeScope& NodeScope, bool bCosmeticChain, const UEdGraphPin* Pin = NULL)
	{
		check(Node != NULL);

		FVisitedTracker* VisitedTracker = NULL;
		if (NodeScope.ContextStack.Num() > 0)
		{
			UK2Node_MacroInstance* MacroInstance = NodeScope.ContextStack.Top();
			VisitedTracker = &GraphNodesVisited.FindOrAdd(MacroInstance);
		}
		else
		{
			// Not a macro, just a plain graph; we use NULL for that.
			VisitedTracker = GraphNodesVisited.Find(nullptr);
			check(VisitedTracker);
		}

		if (!VisitedTracker->IsVisited(Node, bCosmeticChain, Pin))
		{
			VisitedTracker->AddNode(Node, bCosmeticChain, Pin);
			NodeScopeStack.Add(FNodeScope(Node, NodeScope.ContextStack, bCosmeticChain, Pin));
			return true;
		}

		return false;
	}
}

/** 
 * Execute the loading of each blueprint
 *
 * @param Parameters - Should specify which blueprint name to load
 * @return	TRUE if the test was successful, FALSE otherwise
 */

bool FTraceAllTimelinesAutomationTest::RunTest(const FString& BlueprintName)
{
	UBlueprint* BlueprintObj = Cast<UBlueprint>(StaticLoadObject(UBlueprint::StaticClass(), NULL, *BlueprintName));
	if (!BlueprintObj || !BlueprintObj->ParentClass )
	{
		UE_LOGF(LogEditorAutomationTests, Error, "Failed to load invalid blueprint, or blueprint parent no longer exists.");
		return false;
	}

	bool bPassed = true;

	// List all timelines
	TArray<UK2Node_Timeline*> AllTimelines;
	FBlueprintEditorUtils::GetAllNodesOfClass<UK2Node_Timeline>(BlueprintObj, AllTimelines);
	if (AllTimelines.Num() > 0)
	{
		// Cached list of all CallFunction nodes in this blueprint.
		bool bFoundCallFunctionNodes = false;
		TArray<UK2Node_CallFunction*> AllCallFunctionNodes;

		for (auto TimelineIter = AllTimelines.CreateIterator(); TimelineIter; ++TimelineIter)
		{
			UK2Node_Timeline* TimelinePtr = *TimelineIter;
			UE_LOGF(LogEditorAutomationTests, Log, "TraceTimeline: %ls [%ls]", *TimelinePtr->GetPathName(), *TimelinePtr->TimelineName.ToString());

			// Walk up the execution chain and find the list of events that can trigger the timeline
			TraceHelper::TNodeScopeStack NodeScopeStack;
			TraceHelper::TGraphNodesVisited GraphNodesVisited;
			typedef TArray<UK2Node_Event*, TInlineAllocator<16>> TEventList;
			TEventList UncheckedEventNodes;
			TEventList CheckedEventNodes;
			TEventList WarningEventNodes;

			// Initial starting node is the Timeline itself.
			NodeScopeStack.Add(TraceHelper::FNodeScope(TimelinePtr, TraceHelper::TContextStack(), true, NULL));
			GraphNodesVisited.Add(NULL, TraceHelper::FVisitedTracker()).AddNode(TimelinePtr, true, NULL);

			while (NodeScopeStack.Num() > 0)
			{
				TraceHelper::FNodeScope CurrentScope = NodeScopeStack.Pop();
				UK2Node* CurrentNode = CurrentScope.Node;

				// Check if this is an event node we're looking for.
				UK2Node_Event* Event = Cast<UK2Node_Event>(CurrentNode);
				if (Event)
				{
					if (UK2Node_CustomEvent* CustomEvent = Cast<UK2Node_CustomEvent>(CurrentNode))
					{
						// Build list of all CallFunction nodes
						if (!bFoundCallFunctionNodes)
						{
							bFoundCallFunctionNodes = true;
							FBlueprintEditorUtils::GetAllNodesOfClass<UK2Node_CallFunction>(BlueprintObj, AllCallFunctionNodes);
						}

						// Expand all CallFunction nodes that reference this event.
						bool bFoundMatch = false;
						for (int32 CallIndex = 0; CallIndex < AllCallFunctionNodes.Num(); ++CallIndex)
						{
							UK2Node_CallFunction* CallFunctionNode = AllCallFunctionNodes[CallIndex];
							if (CustomEvent->GetFunctionName() == CallFunctionNode->FunctionReference.GetMemberName())
							{
								bFoundMatch = true;
								TraceHelper::AddNode(CallFunctionNode, NodeScopeStack, GraphNodesVisited, CurrentScope, CurrentScope.bCosmeticChain);
							}
							else if (CallFunctionNode->FunctionReference.GetMemberName() == TraceHelper::SetTimerName)
							{
								UEdGraphPin* FunctionPin = CallFunctionNode->FindPin(TraceHelper::SetTimerFunctionFieldName);
								if (FunctionPin && CustomEvent->GetFunctionName().ToString() == FunctionPin->DefaultValue)
								{
									bFoundMatch = true;
									TraceHelper::AddNode(CallFunctionNode, NodeScopeStack, GraphNodesVisited, CurrentScope, CurrentScope.bCosmeticChain);
								}
							}
						}

						if (!bFoundMatch)
						{
							WarningEventNodes.AddUnique(CustomEvent);
						}

						continue;
					}
					else
					{
						// This is a native event, which is an entry point to the BP
						if (CurrentScope.bCosmeticChain)
						{
							UncheckedEventNodes.AddUnique(Event);
						}
						else
						{
							CheckedEventNodes.AddUnique(Event);
						}

						continue;
					}
				}
				else if (UK2Node_Tunnel* Tunnel = Cast<UK2Node_Tunnel>(CurrentNode))
				{
					// Handle tunnel nodes (collapsed graphs and macros)
					if (UK2Node_MacroInstance* MacroInstance = Cast<UK2Node_MacroInstance>(CurrentNode))
					{
						// Enter a macro
						if (UEdGraph* MacroGraph = MacroInstance->GetMacroGraph())
						{
							// We don't want to expand into this special macro.
							if (MacroGraph->GetFName() != TraceHelper::CosmeticMacroName)
							{
								// Jump to the output node of the macro
								TArray<UK2Node_Tunnel*> TunnelNodes;
								MacroGraph->GetNodesOfClass(TunnelNodes);
								for (int32 i = 0; i < TunnelNodes.Num(); i++)
								{
									UK2Node_Tunnel* Node = TunnelNodes[i];
									if (Node->bCanHaveInputs && !Node->bCanHaveOutputs)
									{
										// Push this macro on the context stack and add the output node.
										CurrentScope.ContextStack.Push(MacroInstance);
										
										// Visit the pin on the node matching the one we are entering the macro with.
										const UEdGraphPin* MatchingPin = CurrentScope.Pin ? Node->FindPin(CurrentScope.Pin->PinName) : NULL;										
										TraceHelper::AddNode(Node, NodeScopeStack, GraphNodesVisited, CurrentScope, CurrentScope.bCosmeticChain, MatchingPin);
										break;
									}
								}

								// Done with this node, we expanded the macro.
								continue;
							}	
						}
					}
					else if (UK2Node_Composite* CompositeNode = Cast<UK2Node_Composite>(CurrentNode))
					{
						// Jump to the output node within the composite graph.
						UK2Node_Tunnel* Node = CompositeNode->GetExitNode();
						if (Node)
						{
							// Visit the pin on the node matching the one we are entering the macro with.
							const UEdGraphPin* MatchingPin = CurrentScope.Pin ? Node->FindPin(CurrentScope.Pin->PinName) : NULL;
							TraceHelper::AddNode(Node, NodeScopeStack, GraphNodesVisited, CurrentScope, CurrentScope.bCosmeticChain, MatchingPin);
						}

						// Done with this node, we expanded the graph.
						continue;
					}
					else if (Tunnel->bCanHaveOutputs && !Tunnel->bCanHaveInputs)
					{
						// Exiting a composite graph or macro
						UK2Node_Tunnel* TunnelSource = Tunnel->GetOutputSource();
						
						// We get a null tunnel source for macros
						if (TunnelSource == NULL)
						{
							TunnelSource = CurrentScope.ContextStack.Pop();
							check(TunnelSource);
						}

						// The tunnel node has input pins we can follow, now that we've dug down through the macro itself.
						CurrentNode = TunnelSource;
						CurrentScope.Node = CurrentNode;
						CurrentScope.Pin = CurrentScope.Pin ? CurrentNode->FindPin(CurrentScope.Pin->PinName) : NULL;

						// Expand the source node immediately (do not restart the loop)
					}
				}

				if (CurrentNode)
				{
					//
					// General Nodes
					// Expand all input exec pins (timelines have more than one, for example)
					//
					for (auto PinIter = CurrentNode->Pins.CreateConstIterator(); PinIter; ++PinIter)
					{
						const UEdGraphPin* CurrentPin = CurrentScope.Pin ? CurrentScope.Pin : *PinIter;
						if (CurrentPin->Direction == EGPD_Input && CurrentPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec)
						{
							for (auto LinkedToIter = CurrentPin->LinkedTo.CreateConstIterator(); LinkedToIter; ++LinkedToIter)
							{
								const UEdGraphPin* OtherPin = *LinkedToIter;
								UK2Node* OtherPinNode = Cast<UK2Node>(OtherPin->GetOuter());
								bool bCosmeticChain = CurrentScope.bCosmeticChain;

								UK2Node_Tunnel* Tunnel = Cast<UK2Node_Tunnel>(OtherPinNode);
								if (Tunnel)
								{
									// See if this is an explicit check for allowable cosmetic actions.
									UK2Node_MacroInstance* MacroInstance = Cast<UK2Node_MacroInstance>(OtherPinNode);
									if (MacroInstance)
									{
										UEdGraph* MacroGraph = MacroInstance->GetMacroGraph();
										if (MacroGraph && MacroGraph->GetFName() == TraceHelper::CosmeticMacroName)
										{
											if (bCosmeticChain)
											{
												// This execution chain is checking that it is safe to execute cosmetic events.
												bCosmeticChain = (OtherPin->PinName != TraceHelper::CosmeticCheckedPinName);
											}
											// Don't bother trying to identify pins on this special node, all it does is change the cosmetic chain state.
											OtherPin = nullptr;
										}
									}
								}

								// We only really care to distinguish pins for tunnel nodes,
								// because those might have different input pins hooked up to the logic ending in this output pin
								TraceHelper::AddNode(OtherPinNode, NodeScopeStack, GraphNodesVisited, CurrentScope, bCosmeticChain, Tunnel ? OtherPin : nullptr);
							}
						}

						// We were restricted to only this one pin.
						if (CurrentScope.Pin)
						{
							break;
						}
					}
				}				
			}

			// Build list of all unique events.
			TEventList AllEventNodes;
			AllEventNodes.Append(UncheckedEventNodes);
			for (int32 EventIndex = 0; EventIndex < CheckedEventNodes.Num(); ++EventIndex)
			{
				AllEventNodes.AddUnique(CheckedEventNodes[EventIndex]);
			}
			AllEventNodes.Append(WarningEventNodes);

			// Now list all the event nodes
			for (int32 EventIndex = 0; EventIndex < AllEventNodes.Num(); ++EventIndex)
			{
				UK2Node_Event* Event = AllEventNodes[EventIndex];
				UFunction* Function = Event->FindEventSignatureFunction();

				const bool bIsCosmeticEvent = (Function && Function->HasAllFunctionFlags(FUNC_BlueprintCosmetic)) || Event->IsCosmeticTickEvent();
				const bool bIsCosmeticChain = UncheckedEventNodes.Contains(Event);
				const bool bIsBadEvent = (!bIsCosmeticEvent && bIsCosmeticChain);
				const bool bIsWarningEvent = WarningEventNodes.Contains(Event);
				const TCHAR SymbolString = bIsBadEvent ? TEXT('-') : (bIsWarningEvent ? TEXT('?') : TEXT('+'));
				const TCHAR* CosmeticString = bIsCosmeticEvent ? TEXT("Client") : TEXT("Server");
				FString OutputText = FString::Printf(TEXT("TraceTimeline:   %c %sEvent '%s' -> %s"), SymbolString, CosmeticString, *Event->GetNodeTitle(ENodeTitleType::EditableTitle).ToString(), *TimelinePtr->TimelineName.ToString());

				if (bIsBadEvent)
				{
					// This is an error if we have not branched on a condition checking whether cosmetic events are allowed.
					UE_LOGF(LogEditorAutomationTests, Error, "%ls", *OutputText);
					bPassed = false;
				}
				else if (bIsWarningEvent)
				{
					UE_LOGF(LogEditorAutomationTests, Warning, "%ls", *(OutputText + TEXT(" (could not find function calling this event)")));
				}

				// I always want it in the log as well as the automation test log.
				UE_LOGF(LogEditorAutomationTests, Log, "%ls", *OutputText);
			}
		}
	}

	return bPassed;
}


/**
* Tool to look for overlapping UV's in static meshes.
*/
IMPLEMENT_COMPLEX_AUTOMATION_TEST(FStaticMeshUVCheck, "Project.Tools.Static Mesh.Static Mesh UVs Check", (EAutomationTestFlags::EditorContext | EAutomationTestFlags::RequiresUser | EAutomationTestFlags::StressFilter));

void FStaticMeshUVCheck::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	//This grabs each Static Mesh in the Game/Content
	FAutomationEditorCommonUtils::CollectGameContentTestsByClass(UStaticMesh::StaticClass(), true, OutBeautifiedNames, OutTestCommands);
}

bool FStaticMeshUVCheck::RunTest(const FString& Parameters)
{
	UObject* Object = StaticLoadObject(UObject::StaticClass(), NULL, *Parameters);

	// Missing UV messages
	TArray<FString> MissingUVMessages;
	// Bas UV messages
	TArray<FString> BadUVMessages;
	// Valid UV messages
	TArray<FString> ValidUVMessages;

	UStaticMesh::CheckLightMapUVs((UStaticMesh*)Object, MissingUVMessages, BadUVMessages, ValidUVMessages, true);

	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

	return true;
}


/**
* Launches a map onto a specified device after making a change to it.
*/
IMPLEMENT_COMPLEX_AUTOMATION_TEST(FLaunchOnTest, "Project.Editor.Launch On Test", (EAutomationTestFlags::EditorContext | EAutomationTestFlags::RequiresUser | EAutomationTestFlags::EngineFilter))

void FLaunchOnTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const
{
	{
		UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
		check(AutomationTestSettings);

		TArray<FString> MapToLaunch;
		TArray<FString> DeviceToUse;
		for (auto Iter = AutomationTestSettings->LaunchOnSettings.CreateConstIterator(); Iter; ++Iter)
		{
			if (Iter->LaunchOnTestmap.FilePath.Len() > 0 && !Iter->DeviceID.IsEmpty())
			{
				MapToLaunch.Add(Iter->LaunchOnTestmap.FilePath);
				DeviceToUse.Add(Iter->DeviceID);
			}

			for (int32 i = 0; i < MapToLaunch.Num(); ++i)
			{
				//Get the location of the map being used.
				FString Filename = FPaths::ConvertRelativePathToFull(MapToLaunch[i]);

				//Get the DeviceID
				FString DeviceID;
				FAutomationEditorCommonUtils::GetLaunchOnDeviceID(DeviceID, FPaths::GetBaseFilename(MapToLaunch[i]), DeviceToUse[i]);

				if (!DeviceID.IsEmpty() && !DeviceID.Equals(TEXT("None")))
				{
					if (Filename.Contains(TEXT("/Engine/"), ESearchCase::IgnoreCase, ESearchDir::FromStart))
					{
						//If true it will proceed to add the asset to the test list.
						//This will be false if the map is on a different drive.
						if (FPaths::MakePathRelativeTo(Filename, *FPaths::EngineContentDir()))
						{
							FString ShortName = FPaths::GetBaseFilename(Filename);
							FString PathName = FPaths::GetPath(Filename);
							FString AssetName = FString::Printf(TEXT("/Game/%s/%s.%s %s"), *PathName, *ShortName, *ShortName, *DeviceID);

							ShortName += (TEXT(" ( ") + DeviceID.Left(DeviceID.Find(TEXT("@"))) + TEXT(" ) "));

							OutBeautifiedNames.Add(ShortName);
							OutTestCommands.Add(AssetName);
						}
						else
						{
							UE_LOGF(LogEditorAutomationTests, Error, "Invalid asset path: %ls.", *Filename);
						}
					}
					else
					{
						//If true it will proceed to add the asset to the test list.
						//This will be false if the map is on a different drive.
						if (FPaths::MakePathRelativeTo(Filename, *FPaths::ProjectContentDir()))
						{
							FString ShortName = FPaths::GetBaseFilename(Filename);
							FString PathName = FPaths::GetPath(Filename);
							FString AssetName = FString::Printf(TEXT("/Game/%s/%s.%s %s"), *PathName, *ShortName, *ShortName, *DeviceID);

							ShortName += (TEXT(" (") + DeviceID.Left(DeviceID.Find(TEXT("@"))) + TEXT(") "));

							OutBeautifiedNames.Add(ShortName);
							OutTestCommands.Add(AssetName);
						}
						else
						{
							UE_LOGF(LogEditorAutomationTests, Error, "Invalid asset path: %ls.", *Filename);
						}
					}
				}
			}
		}
	}
}

bool FLaunchOnTest::RunTest(const FString& Parameters)
{
	//Get the map name and device id from the parameters.
	FString MapName = Parameters.Left(Parameters.Find(TEXT(" ")));
	FString DeviceID = Parameters.RightChop(Parameters.Find(TEXT(" ")));
	DeviceID.TrimStartInline();

	//Delete the Cooked, StagedBuilds, and Automation_TEMP folder if they exist.
	FString CookedLocation = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("Cooked"));
	FString StagedBuildsLocation = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("StagedBuilds"));
	FString TempMapLocation = FPaths::Combine(*FPaths::ProjectContentDir(), TEXT("Maps"), TEXT("Automation_TEMP"));
	ADD_LATENT_AUTOMATION_COMMAND(FDeleteDirCommand(CookedLocation));
	ADD_LATENT_AUTOMATION_COMMAND(FDeleteDirCommand(StagedBuildsLocation));
	ADD_LATENT_AUTOMATION_COMMAND(FDeleteDirCommand(TempMapLocation));
	ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(2.0f));

	//Load Map and get the time it took to take to load the map.
	ADD_LATENT_AUTOMATION_COMMAND(FEditorLoadMap(MapName));
	ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(1.0f));

	//Make an adjustment to the map and rebuild its lighting.
	ADD_LATENT_AUTOMATION_COMMAND(FAddStaticMeshCommand);
	ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(1.0f));
	ADD_LATENT_AUTOMATION_COMMAND(FBuildLightingCommand);

	//Save a copy of the map to the automation temp map folder location once the lighting build has finish.
	ADD_LATENT_AUTOMATION_COMMAND(FSaveLevelCommand(FPaths::GetBaseFilename(MapName)));

	//Launch onto device and get launch on times and cook times
	ADD_LATENT_AUTOMATION_COMMAND(FLaunchOnCommand(DeviceID));
	ADD_LATENT_AUTOMATION_COMMAND(FWaitToFinishCookByTheBookCommand);
	ADD_LATENT_AUTOMATION_COMMAND(FWaitToFinishBuildDeployCommand);

	//@todo: Verify the game launched.

	//@todo: Close the Launched on Game.

	//Delete the Cooked, StagedBuilds, and Automation_TEMP folder if they exist.
	ADD_LATENT_AUTOMATION_COMMAND(FDeleteDirCommand(CookedLocation));
	ADD_LATENT_AUTOMATION_COMMAND(FDeleteDirCommand(StagedBuildsLocation));
	ADD_LATENT_AUTOMATION_COMMAND(FDeleteDirCommand(TempMapLocation));

	return true;
}

