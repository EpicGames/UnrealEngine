// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/AutomationTest.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "UObject/PropertyPortFlags.h"
#include "Misc/PackageName.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/InputChord.h"
#include "EngineGlobals.h"
#include "EdGraph/EdGraphPin.h"
#include "Engine/Blueprint.h"
#include "GameFramework/Actor.h"
#include "Materials/MaterialInterface.h"
#include "Engine/SkeletalMesh.h"
#include "Components/StaticMeshComponent.h"
#include "AI/NavigationSystemBase.h"
#include "Engine/Texture.h"
#include "Engine/StaticMesh.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Materials/Material.h"
#include "Factories/Factory.h"
#include "AssetRegistry/AssetData.h"
#include "EdGraph/EdGraph.h"
#include "Sound/SoundWave.h"
#include "GameFramework/WorldSettings.h"
#include "Engine/DirectionalLight.h"
#include "Engine/PointLight.h"
#include "Engine/StaticMeshActor.h"
#include "EditorReimportHandler.h"
#include "Animation/AnimSequence.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlModule.h"
#include "SourceControlHelpers.h"
#include "Editor/UnrealEdEngine.h"
#include "Factories/BlueprintFactory.h"
#include "Factories/FbxFactory.h"
#include "Factories/SoundFactory.h"
#include "Factories/TextureFactory.h"
#include "Factories/FbxImportUI.h"
#include "Settings/LevelEditorMiscSettings.h"
#include "Editor.h"
#include "FileHelpers.h"
#include "UnrealEdGlobals.h"
#include "UObject/SavePackage.h"

//Automation
#include "Tests/AutomationTestSettings.h"
#include "Tests/AutomationCommon.h"
#include "Tests/AutomationEditorCommon.h"
#include "Tests/AutomationEditorPromotionCommon.h"

//Assets
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetSelection.h"
#include "PackageHelperFunctions.h"

//Materials

#include "Editor/MaterialEditor/Private/MaterialEditor.h"
#include "Materials/MaterialExpressionConstant3Vector.h"

//Particle system

//Blueprints
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "K2Node_Event.h"
#include "K2Node_CallFunction.h"
#include "K2Node_CustomEvent.h"
#include "EdGraphUtilities.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Kismet2/Breakpoint.h"
#include "Engine/LevelScriptBlueprint.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"

//Utils
#include "EditorLevelUtils.h"

//Source Control
#include "Tests/SourceControlAutomationCommon.h"

//Lighting
#include "LightingBuildOptions.h"

//Animation

//Audio

#include "LevelEditor.h"
#include "ObjectTools.h"
#include "Subsystems/AssetEditorSubsystem.h"


#define LOCTEXT_NAMESPACE "EditorBuildPromotionTests"

DEFINE_LOG_CATEGORY_STATIC(LogEditorBuildPromotionTests, Log, All);

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBuildPromotionInitialCleanupTest, "System.Promotion.Editor Promotion Pass.Step 1 Main Editor Test.Cleanup old files", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditorPromotionTest, "System.Promotion.Editor Promotion Pass.Step 1 Main Editor Test.General Editor Test", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBuildPromotionPIETest, "System.Promotion.Editor Promotion Pass.Step 2 Run Map After Re-launch.Run Map", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBuildPromotionCleanupTest, "System.Promotion.Editor Promotion Pass.Step 3 Test Cleanup.Cleanup", EAutomationTestFlags::Disabled | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

/**
* Helper functions used by the build promotion automation test
*/
namespace EditorBuildPromotionTestUtils
{
	/**
	* Constants
	*/
	static const FString BlueprintNameString = TEXT("EditorPromotionBlueprint");
	static const FName BlueprintStringVariableName(TEXT("MyStringVariable"));

	/**
	* Gets the full path to the folder on disk
	*/
	static FString GetFullPath()
	{
		return FPackageName::FilenameToLongPackageName(FPaths::ProjectContentDir() + TEXT("EditorPromotionTest"));
	}

	/**
	* Helper class to track once a certain time has passed
	*/
	class FDelayHelper
	{
	public:
		/**
		* Constructor
		*/
		FDelayHelper() :
			bIsRunning(false),
			StartTime(0.f),
			Duration(0.f)
		{
		}

		/**
		* Returns if the delay is still running
		*/
		bool IsRunning()
		{
			return bIsRunning;
		}

		/**
		* Sets the helper state to not running
		*/
		void Reset()
		{
			bIsRunning = false;
		}

		/**
		* Starts the delay timer
		*
		* @param InDuration - How long to delay for in seconds
		*/
		void Start(double InDuration)
		{
			bIsRunning = true;
			StartTime = FPlatformTime::Seconds();
			Duration = InDuration;
		}

		/**
		* Returns true if the desired amount of time has passed
		*/
		bool IsComplete()
		{
			if (IsRunning())
			{
				const double CurrentTime = FPlatformTime::Seconds();
				return CurrentTime - StartTime >= Duration;
			}
			else
			{
				return false;
			}
		}

	private:
		/** If true, this delay timer is active */
		bool bIsRunning;
		/** The time the delay started */
		double StartTime;
		/** How long the timer is for */
		double Duration;
	};

	/**
	* Sends the MaterialEditor->Apply UI command
	*/
	static void SendUpdateMaterialCommand()
	{
		const FString Context = TEXT("MaterialEditor");
		const FString Command = TEXT("Apply");
		FInputChord CurrentApplyChord = FEditorPromotionTestUtilities::GetOrSetUICommand(Context, Command);

		const FName FocusWidgetType(TEXT("SGraphEditor"));
		FEditorPromotionTestUtilities::SendCommandToCurrentEditor(CurrentApplyChord, FocusWidgetType);
	}

	/**
	* Compiles the blueprint
	*
	* @param InBlueprint - The blueprint to compile
	*/
	static void CompileBlueprint(UBlueprint* InBlueprint)
	{
		FBlueprintEditorUtils::RefreshAllNodes(InBlueprint);

		FKismetEditorUtilities::CompileBlueprint(InBlueprint, EBlueprintCompileOptions::SkipGarbageCollection);
		if (InBlueprint->Status == EBlueprintStatus::BS_UpToDate)
		{
			UE_LOGF(LogEditorBuildPromotionTests, Display, "Blueprint compiled successfully (%ls)", *InBlueprint->GetName());
		}
		else if (InBlueprint->Status == EBlueprintStatus::BS_UpToDateWithWarnings)
		{
			UE_LOGF(LogEditorBuildPromotionTests, Display, "Blueprint compiled successfully with warnings(%ls)", *InBlueprint->GetName());
		}
		else if (InBlueprint->Status == EBlueprintStatus::BS_Error)
		{
			UE_LOGF(LogEditorBuildPromotionTests, Display, "Blueprint failed to compile (%ls)", *InBlueprint->GetName());
		}
		else
		{
			UE_LOGF(LogEditorBuildPromotionTests, Error, "Blueprint is in an unexpected state after compiling (%ls)", *InBlueprint->GetName());
		}
	}

	/**
	* Creates a new graph node from a given template
	*
	* @param NodeTemplate - The template to use for the node
	* @param InGraph - The graph to create the new node in
	* @param GraphLocation - The location to place the node
	* @param ConnectPin - The pin to connect the node to
	*/
	static UEdGraphNode* CreateNewGraphNodeFromTemplate(UK2Node* NodeTemplate, UEdGraph* InGraph, const FVector2f& GraphLocation, UEdGraphPin* ConnectPin = NULL)
	{
		TSharedPtr<FEdGraphSchemaAction_K2NewNode> Action = TSharedPtr<FEdGraphSchemaAction_K2NewNode>(new FEdGraphSchemaAction_K2NewNode(FText::GetEmpty(), FText::GetEmpty(), FText::GetEmpty(), 0));
		Action->NodeTemplate = NodeTemplate;

		return Action->PerformAction(InGraph, ConnectPin, GraphLocation, false);
	}

	/**
	* Creates a ReceiveBeginPlay event node
	*
	* @param InBlueprint - The blueprint to modify
	* @param InGraph - The graph to use for the new node
	*/
	static UEdGraphNode* CreatePostBeginPlayEvent(UBlueprint* InBlueprint, UEdGraph* InGraph)
	{
		UEdGraph* TempOuter = NewObject<UEdGraph>((UObject*)InBlueprint);
		TempOuter->SetFlags(RF_Transient);

		// Make an add component node
		UK2Node_Event* NewEventNode = NewObject<UK2Node_Event>(TempOuter);
		NewEventNode->EventReference.SetExternalMember(FName(TEXT("ReceiveBeginPlay")), AActor::StaticClass());
		NewEventNode->bOverrideFunction = true;

		//Check for existing events
		UK2Node_Event* ExistingEvent = FBlueprintEditorUtils::FindOverrideForFunction(InBlueprint, NewEventNode->EventReference.GetMemberParentClass(NewEventNode->GetBlueprintClassFromNode()), NewEventNode->EventReference.GetMemberName());

		if (!ExistingEvent)
		{
			return CreateNewGraphNodeFromTemplate(NewEventNode, InGraph, FVector2f(200.0f, 0.0f));
		}
		return ExistingEvent;
	}

	/**
	* Creates a custom event node
	*
	* @param InBlueprint - The blueprint to modify
	* @param InGraph - The graph to use for the new node
	* @param EventName - The name of the event
	*/
	static UEdGraphNode* CreateCustomEvent(UBlueprint* InBlueprint, UEdGraph* InGraph, const FString& EventName)
	{
		UEdGraph* TempOuter = NewObject<UEdGraph>((UObject*)InBlueprint);
		TempOuter->SetFlags(RF_Transient);

		// Make an add component node
		UK2Node_CustomEvent* NewEventNode = NewObject<UK2Node_CustomEvent>(TempOuter);
		NewEventNode->CustomFunctionName = "EventName";

		return CreateNewGraphNodeFromTemplate(NewEventNode, InGraph, FVector2f(1200.0f, 0.0f));
	}

	/**
	* Creates a node template for a UKismetSystemLibrary function
	*
	* @param NodeOuter - The outer to use for the template
	* @param InGraph - The function to use for the node
	*/
	static UK2Node* CreateKismetFunctionTemplate(UObject* NodeOuter, const FName& FunctionName)
	{
		// Make a call function template
		UK2Node_CallFunction* CallFuncNode = NewObject<UK2Node_CallFunction>(NodeOuter);
		UFunction* Function = FindFieldChecked<UFunction>(UKismetSystemLibrary::StaticClass(), FunctionName);
		CallFuncNode->FunctionReference.SetFromField<UFunction>(Function, false);
		return CallFuncNode;
	}

	/**
	* Creates a PrintString node
	*
	* @param InBlueprint - The blueprint to modify
	* @param InGraph - The graph to use for the new node
	* @param ConnectPin - The pin to connect the new node to
	*/
	static UEdGraphNode* AddPrintStringNode(UBlueprint* InBlueprint, UEdGraph* InGraph, UEdGraphPin* ConnectPin = NULL)
	{
		UEdGraph* TempOuter = NewObject<UEdGraph>((UObject*)InBlueprint);
		TempOuter->SetFlags(RF_Transient);

		// Make a call function template
		static FName PrintStringFunctionName(TEXT("PrintString"));
		UK2Node* CallFuncNode = CreateKismetFunctionTemplate(TempOuter, PrintStringFunctionName);

		return CreateNewGraphNodeFromTemplate(CallFuncNode, InGraph, FVector2f(680.0f, 0.0f), ConnectPin);
	}

	/**
	* Starts a lighting build
	*/
	static void BuildLighting()
	{
		//If we are running with -NullRHI then we have to skip this step.
		if (GUsingNullRHI)
		{
			UE_LOGF(LogEditorBuildPromotionTests, Log, "SKIPPED Build Lighting Step.  You're currently running with -NullRHI.");
			return;
		}

		UWorld* CurrentWorld = GEditor->GetEditorWorldContext().World();
		GUnrealEd->Exec(CurrentWorld, TEXT("MAP REBUILD"));

		FLightingBuildOptions LightingBuildOptions;

		// Retrieve settings from ini.
		GConfig->GetBool(TEXT("LightingBuildOptions"), TEXT("OnlyBuildSelected"), LightingBuildOptions.bOnlyBuildSelected, GEditorPerProjectIni);
		GConfig->GetBool(TEXT("LightingBuildOptions"), TEXT("OnlyBuildCurrentLevel"), LightingBuildOptions.bOnlyBuildCurrentLevel, GEditorPerProjectIni);
		GConfig->GetBool(TEXT("LightingBuildOptions"), TEXT("OnlyBuildSelectedLevels"), LightingBuildOptions.bOnlyBuildSelectedLevels, GEditorPerProjectIni);
		GConfig->GetBool(TEXT("LightingBuildOptions"), TEXT("OnlyBuildVisibility"), LightingBuildOptions.bOnlyBuildVisibility, GEditorPerProjectIni);
		GConfig->GetBool(TEXT("LightingBuildOptions"), TEXT("UseErrorColoring"), LightingBuildOptions.bUseErrorColoring, GEditorPerProjectIni);
		GConfig->GetBool(TEXT("LightingBuildOptions"), TEXT("ShowLightingBuildInfo"), LightingBuildOptions.bShowLightingBuildInfo, GEditorPerProjectIni);
		int32 QualityLevel;
		GConfig->GetInt(TEXT("LightingBuildOptions"), TEXT("QualityLevel"), QualityLevel, GEditorPerProjectIni);
		QualityLevel = FMath::Clamp<int32>(QualityLevel, Quality_Preview, Quality_Production);
		LightingBuildOptions.QualityLevel = (ELightingBuildQuality)QualityLevel;

		GUnrealEd->BuildLighting(LightingBuildOptions);
	}

	/**
	* Gets an object property value by name
	*
	* @param TargetObject - The object to modify
	* @param InVariableName - The name of the property
	*/
	static FString GetPropertyByName(UObject* TargetObject, const FString& InVariableName)
	{
		FProperty* FoundProperty = FindFProperty<FProperty>(TargetObject->GetClass(), *InVariableName);
		if (FoundProperty)
		{
			FString ValueString;
			const uint8* PropertyAddr = FoundProperty->ContainerPtrToValuePtr<uint8>(TargetObject);
			FoundProperty->ExportTextItem_Direct(ValueString, PropertyAddr, NULL, NULL, PPF_None);
			return ValueString;
		}
		return TEXT("");
	}

	/**
	* Starts a PIE session
	*/
	static void StartPIE(bool bSimulateInEditor)
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::Get().GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		TSharedPtr<class IAssetViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveViewport();
		
		FRequestPlaySessionParams SessionParams;
		SessionParams.DestinationSlateViewport = ActiveLevelViewport;
		if (bSimulateInEditor)
		{
			SessionParams.WorldType = EPlaySessionWorldType::SimulateInEditor;
		}

		GUnrealEd->RequestPlaySession(SessionParams);
	}

	/**
	* Adds a default mesh to the level
	*
	* @param Location - The location to place the actor
	*/
	static AStaticMeshActor* AddDefaultMeshToLevel(const FVector& Location)
	{
		UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
		check(AutomationTestSettings);

		// Default static mesh
		FString AssetPackagePath = AutomationTestSettings->BuildPromotionTest.DefaultStaticMeshAsset.FilePath;
		if (AssetPackagePath.Len() > 0)
		{
			FAssetData AssetData = FAutomationEditorCommonUtils::GetAssetDataFromPackagePath(AssetPackagePath);
			UStaticMesh* DefaultMesh = Cast<UStaticMesh>(AssetData.GetAsset());
			if (DefaultMesh)
			{
				AStaticMeshActor* PlacedMesh = Cast<AStaticMeshActor>(FActorFactoryAssetProxy::AddActorForAsset(DefaultMesh));
				PlacedMesh->SetActorLocation(Location);

				return PlacedMesh;
			}
			else
			{
				UE_LOGF(LogEditorBuildPromotionTests, Error, "DefaultStaticMeshAsset is invalid.");
			}
		}
		else
		{
			UE_LOGF(LogEditorBuildPromotionTests, Warning, "Can't add Static Mesh to level because no DefaultMeshAsset is defined.");
		}

		return NULL;
	}

	/*
	* Applies a material to a static mesh. Triggers a test failure if StaticMesh is not valid.
	*
	* @param StaticMesh - the static mesh to apply the material to
	* @param Material - the material to apply
	*/
	static bool ApplyMaterialToStaticMesh(AStaticMeshActor* StaticMesh, UMaterialInterface* Material)
	{
		if (StaticMesh)
		{
			StaticMesh->GetStaticMeshComponent()->SetMaterial(0, Material);
			return true;
		}
		else
		{
			UE_LOGF(LogEditorBuildPromotionTests, Warning, "Failed to apply material to static mesh because mesh does not exist");
			return false;
		}
	}

	/**
	* Imports an asset using the supplied factory and file
	*
	* @param ImportFactory - The factory to use to import the asset
	* @param ImportPath - The file path of the file to use
	*/
	static UObject* ImportAsset(UFactory* ImportFactory, const FString& ImportPath)
	{
		const FString Name = ObjectTools::SanitizeObjectName(FPaths::GetBaseFilename(ImportPath));
		const FString PackageName = FString::Printf(TEXT("%s/%s"), *FEditorPromotionTestUtilities::GetGamePath(), *Name);

		UObject* ImportedAsset = FAutomationEditorCommonUtils::ImportAssetUsingFactory(ImportFactory,Name,PackageName,ImportPath);

		return ImportedAsset;
	}

	static void PlaceImportedAsset(UObject* InObject, FVector& PlaceLocation)
	{
		if (InObject)
		{
			AActor* PlacedActor = NULL;
			UTexture* TextureObject = Cast<UTexture>(InObject);
			if (TextureObject)
			{
				//Don't add if we are a normal map
				if (TextureObject->LODGroup == TEXTUREGROUP_WorldNormalMap)
				{
					return;
				}
				else
				{
					UMaterial* NewMaterial = FEditorPromotionTestUtilities::CreateMaterialFromTexture(TextureObject);
					PlacedActor = EditorBuildPromotionTestUtils::AddDefaultMeshToLevel(PlaceLocation);
					EditorBuildPromotionTestUtils::ApplyMaterialToStaticMesh(Cast<AStaticMeshActor>(PlacedActor), NewMaterial);
				}
			}
			else
			{
				PlacedActor = FActorFactoryAssetProxy::AddActorForAsset(InObject);
			}

			if (PlacedActor)
			{
				PlacedActor->SetActorLocation(PlaceLocation);
				PlaceLocation.Y += 100;
				UE_LOGF(LogEditorBuildPromotionTests, Display, "Placed %ls in the level", *InObject->GetName());
			}
			else
			{
				UE_LOGF(LogEditorBuildPromotionTests, Error, "Failed to place %ls in the level", *InObject->GetName());
			}
		}
	}

	/**
	* Saves all assets in a given folder
	*
	* @param InFolder - The folder that contains the assets to save
	*/
	static void SaveAllAssetsInFolder(const FString& InFolder)
	{
		// Load the asset registry module
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

		// Form a filter from the paths
		FARFilter Filter;
		Filter.bRecursivePaths = true;
		new (Filter.PackagePaths) FName(*InFolder);
		

		// Query for a list of assets in the selected paths
		TArray<FAssetData> AssetList;
		AssetRegistryModule.Get().GetAssets(Filter, AssetList);

		// Form a list of unique package names from the assets
		TSet<FName> UniquePackageNames;
		for (int32 AssetIdx = 0; AssetIdx < AssetList.Num(); ++AssetIdx)
		{
			UniquePackageNames.Add(AssetList[AssetIdx].PackageName);
		}

		// Add all unique package names to the output
		TArray<FString> PackageNames;
		for (auto PackageIt = UniquePackageNames.CreateConstIterator(); PackageIt; ++PackageIt)
		{
			PackageNames.Add((*PackageIt).ToString());
		}

		// Form a list of packages from the assets
		TArray<UPackage*> Packages;
		for (int32 PackageIdx = 0; PackageIdx < PackageNames.Num(); ++PackageIdx)
		{
			UPackage* Package = FindPackage(NULL, *PackageNames[PackageIdx]);

			// Only save loaded and dirty packages
			if (Package != NULL && Package->IsDirty())
			{
				Packages.Add(Package);
			}
		}

		// Save all packages that were found
		if (Packages.Num())
		{
			if (FApp::IsUnattended())
			{
				// When unattended, prompt for checkout and save does not work.
				// Save the packages directly instead
				for (UPackage* Package : Packages)
				{
					const bool bIsMapPackage = UWorld::FindWorldInPackage(Package) != nullptr;
					const FString& FileExtension = bIsMapPackage ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension();
					const FString Filename = FPackageName::LongPackageNameToFilename(Package->GetName(), FileExtension);
					SavePackageHelper(Package, Filename);
				}
			}
			else
			{
				const bool bCheckDirty = false;
				const bool bPromptToSave = false;
				FEditorFileUtils::PromptForCheckoutAndSave(Packages, bCheckDirty, bPromptToSave);
			}
		}
	}

	/**
	* Cleans up objects created by the main build promotion test
	*/
	static void PerformCleanup()
	{
		//Make sure we don't have any level references
		FAutomationEditorCommonUtils::CreateNewMap();

		// Deselect all
		GEditor->SelectNone(false, true);

		// Clear the transaction buffer so we aren't referencing the objects
		GUnrealEd->ResetTransaction(FText::FromString(TEXT("FAssetEditorTest")));

		//remove all assets in the build package
		// Load the asset registry module
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

		// Form a filter from the paths
		FARFilter Filter;
		Filter.bRecursivePaths = true;
		new (Filter.PackagePaths) FName(*FEditorPromotionTestUtilities::GetGamePath());

		// Query for a list of assets in the selected paths
		TArray<FAssetData> AssetList;
		AssetRegistry.GetAssets(Filter, AssetList);

		// Clear and try to delete all assets
		for (int32 AssetIdx = 0; AssetIdx < AssetList.Num(); ++AssetIdx)
		{
			UE_LOGF(LogEditorBuildPromotionTests, Display, "Removing asset: %ls", *AssetList[AssetIdx].AssetName.ToString());
			if (AssetList[AssetIdx].IsAssetLoaded())
			{
				UObject* LoadedAsset = AssetList[AssetIdx].GetAsset();
				AssetRegistry.AssetDeleted(LoadedAsset);

				bool bSuccessful = ObjectTools::DeleteSingleObject(LoadedAsset, false);

				//If we failed to delete this object manually clear any references and try again
				if (!bSuccessful)
				{
					//Clear references to the object so we can delete it
					FAutomationEditorCommonUtils::NullReferencesToObject(LoadedAsset);

					bSuccessful = ObjectTools::DeleteSingleObject(LoadedAsset, false);
				}
			}
		}

		UE_LOGF(LogEditorBuildPromotionTests, Display, "Clearing Path: %ls", *FEditorPromotionTestUtilities::GetGamePath());
		AssetRegistry.RemovePath(FEditorPromotionTestUtilities::GetGamePath());

		//Remove the directory
		bool bEnsureExists = false;
		bool bDeleteEntireTree = true;
		FString PackageDirectory = FPaths::ProjectContentDir() / TEXT("BuildPromotionTest");
		IFileManager::Get().DeleteDirectory(*PackageDirectory, bEnsureExists, bDeleteEntireTree);
		UE_LOGF(LogEditorBuildPromotionTests, Display, "Deleting Folder: %ls", *PackageDirectory);

		//Remove the map
		FString MapFilePath = FPaths::ProjectContentDir() / TEXT("Maps/EditorBuildPromotionTest.umap");
		IFileManager::Get().Delete(*MapFilePath, false, true, true);
		UE_LOGF(LogEditorBuildPromotionTests, Display, "Deleting Map: %ls", *MapFilePath);
	}

	/**
	* Gets the number of actors in the persistent level for a given UWorld
	*
	* @param InWorld - The world to check for actors
	* @param ActorType - The type of actors to look for
	*/
	static int32 GetNumActors(UWorld* InWorld,const UClass* ActorType)
	{
		int32 NumActors = 0;
		ULevel* PersistentLevel = InWorld->PersistentLevel;
		for (int32 i = 0; i < PersistentLevel->Actors.Num(); ++i)
		{
			if (PersistentLevel->Actors[i]->IsA(ActorType))
			{
				NumActors++;
			}
		}
		return NumActors;
	}
}



/**
* Container for items related to the create asset test
*/
namespace BuildPromotionTestHelper
{
	/** The possible states of the FOpenAssetInfo class */
	namespace EState
	{
		enum Type
		{
			OpenEditor,
			WaitForEditor,
			ChangeProperty
		};
	}

	/** Stores info on an asset that we are opening */
	struct FOpenAssetInfo
	{
		/** The asset we are opening */
		UObject* Asset;
		/** The asset data  */
		FAssetData AssetData;
		/** The name of the property we are going to change */
		FString PropertyName;
		/** The new value to assign to the property */
		FString PropertyValue;

		/**
		* Constructor
		*/
		FOpenAssetInfo(UObject* InAsset, const FAssetData& InAssetData, const FString& InPropertyName, const FString& InPropertyValue) :
			Asset(InAsset),
			AssetData(InAssetData),
			PropertyName(InPropertyName),
			PropertyValue(InPropertyValue)
		{}
	};

	/** Helper class to open, modify, and add an asset to the level */
	class FOpenAssetHelper
	{
	public:
		/**
		* Constructor
		*/
		FOpenAssetHelper(TArray<FOpenAssetInfo> InAssets, FAutomationTestExecutionInfo* InTestExecutionInfo)
			: Assets(InAssets)
			, CurrentStage(EState::OpenEditor)
			, AssetIndex(-1)
			, TestExecutionInfo(InTestExecutionInfo)
		{
			NextAsset();
		}

		/**
		* Updates the current stage
		*/
		bool Update()
		{
			if (AssetIndex < Assets.Num())
			{
				switch (CurrentStage)
				{
				case EState::OpenEditor:
					OpenAssetEditor();
					break;
				case EState::WaitForEditor:
					WaitForEditor();
					break;
				case EState::ChangeProperty:
					ChangeProperty();
					break;
				}
				return false;
			}
			else
			{
				return true;
			}
		}
	private:

		/**
		* Opens the asset editor
		*/
		void OpenAssetEditor()
		{
			UObject* CurrentAsset = Assets[AssetIndex].Asset;
			if (GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(CurrentAsset))
			{
				CurrentStage = EState::WaitForEditor;
			}
			else
			{
				UE_LOGF(LogEditorBuildPromotionTests, Error, "Failed to open the asset editor!");

				//Move on to the next asset
				NextAsset();
			}
		}

		/**
		* Waits for the asset editor to open
		*/
		void WaitForEditor()
		{
			UObject* CurrentAsset = Assets[AssetIndex].Asset;
			TSharedPtr<SWindow> ActiveWindow = FSlateApplication::Get().GetActiveTopLevelWindow();
			if (ActiveWindow.IsValid())
			{
				FString ActiveWindowTitle = ActiveWindow->GetTitle().ToString();

				//Check that we have the right window (Tutorial may have opened on top of the editor)
				if (!ActiveWindowTitle.StartsWith(CurrentAsset->GetName()))
				{
					//Bring the asset editor to the front
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(CurrentAsset, true);
				}

				UE_LOGF(LogEditorBuildPromotionTests, Display, "Opened asset (%ls)", *CurrentAsset->GetClass()->GetName());

				CurrentStage = EState::ChangeProperty;
			}
			else
			{
				WaitingForEditorCount++;
				if (WaitingForEditorCount > MaxWaitForEditorTicks)
				{

					UE_LOGF(LogEditorBuildPromotionTests, Error, "Timed out waiting for editor window");

					//Move on the next asset
					NextAsset();
				}
			}
		}

		/**
		* Modifies a property on the current asset, undoes and redoes the property change, then saves changed asset
		*/
		void ChangeProperty()
		{
			UObject* CurrentAsset = Assets[AssetIndex].Asset;
			FString PropertyName = Assets[AssetIndex].PropertyName;
			FString NewPropertyValue = Assets[AssetIndex].PropertyValue;

			FString OldPropertyValue = FEditorPromotionTestUtilities::GetPropertyByName(CurrentAsset, PropertyName);
			FEditorPromotionTestUtilities::SetPropertyByName(CurrentAsset, PropertyName, NewPropertyValue);
			UE_LOGF(LogEditorBuildPromotionTests, Display, "Modified asset.  %ls = %ls", *PropertyName, *NewPropertyValue);

			//Get the property again and use that to compare the redo action.  Parsing the new value may change the formatting a bit. ie) 100 becomes 100.0000
			FString ParsedNewValue = FEditorPromotionTestUtilities::GetPropertyByName(CurrentAsset, PropertyName);

			GEditor->UndoTransaction();
			FString CurrentValue = FEditorPromotionTestUtilities::GetPropertyByName(CurrentAsset, PropertyName);
			if (CurrentValue == OldPropertyValue)
			{
				UE_LOGF(LogEditorBuildPromotionTests, Display, "Undo %ls change successful", *PropertyName);
			}
			else
			{
				UE_LOGF(LogEditorBuildPromotionTests, Error, "Undo didn't change property back to original value");
			}

			GEditor->RedoTransaction();
			CurrentValue = FEditorPromotionTestUtilities::GetPropertyByName(CurrentAsset, PropertyName);
			if (CurrentValue == ParsedNewValue)
			{
				UE_LOGF(LogEditorBuildPromotionTests, Display, "Redo %ls change successful", *PropertyName);
			}
			else
			{
				UE_LOGF(LogEditorBuildPromotionTests, Error, "Redo didnt' change property back to the modified value");
			}

			//Apply if this is a material.  (Editor won't close unless we do)
			if (CurrentAsset && CurrentAsset->IsA(UMaterialInterface::StaticClass()))
			{
				EditorBuildPromotionTestUtils::SendUpdateMaterialCommand();
			}

			

			//Save
			const FString PackagePath = Assets[AssetIndex].AssetData.PackageName.ToString();
			if (CurrentAsset && PackagePath.Len() > 0)
			{

				UPackage* AssetPackage = FindPackage(NULL, *PackagePath);
				if (AssetPackage)
				{
					AssetPackage->SetDirtyFlag(true);
					FString PackageFileName = FPackageName::LongPackageNameToFilename(PackagePath, FPackageName::GetAssetPackageExtension());
					FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*PackageFileName, false);
					FSavePackageArgs SaveArgs;
					SaveArgs.TopLevelFlags = RF_Standalone;
					SaveArgs.SaveFlags = SAVE_NoError;
					if (UPackage::SavePackage(AssetPackage, NULL, *PackageFileName, SaveArgs))
					{
						UE_LOGF(LogEditorBuildPromotionTests, Display, "Saved asset");
					}
					else
					{
						UE_LOGF(LogEditorBuildPromotionTests, Display, "Unable to save asset");
					}
				}
			}

			//close editor
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllAssetEditors();

			//Add to level
			FVector PlaceLocation(-1090,970,160);
			PlaceLocation.Y += AssetIndex * 150.f;
			UMaterialInterface* MaterialAsset = Cast<UMaterialInterface>(CurrentAsset);
			UTexture* TextureAsset = Cast<UTexture>(CurrentAsset);
			if (MaterialAsset)
			{
				AStaticMeshActor* PlacedActor = EditorBuildPromotionTestUtils::AddDefaultMeshToLevel(PlaceLocation); // , MaterialAsset);
				EditorBuildPromotionTestUtils::ApplyMaterialToStaticMesh(PlacedActor, MaterialAsset);
			}
			else if (TextureAsset)
			{
				UMaterial* NewMaterial = FEditorPromotionTestUtilities::CreateMaterialFromTexture(TextureAsset);
				AStaticMeshActor* PlacedActor = EditorBuildPromotionTestUtils::AddDefaultMeshToLevel(PlaceLocation); //, NewMaterial);
				EditorBuildPromotionTestUtils::ApplyMaterialToStaticMesh(PlacedActor, NewMaterial);
			}
			else
			{
				AActor* PlacedActor = FActorFactoryAssetProxy::AddActorForAsset(CurrentAsset);
				if (PlacedActor)
				{
					UE_LOGF(LogEditorBuildPromotionTests, Display, "Placed %ls in the level", *GetNameSafe(CurrentAsset));
					PlacedActor->SetActorLocation(PlaceLocation);
				}
				else
				{
					UE_LOGF(LogEditorBuildPromotionTests, Warning, "Unable to place asset");
				}
			}

			NextAsset();
		}

		/**
		* Switches to the next asset in the list
		*/
		void NextAsset()
		{
			TestExecutionInfo->PopContext();

			AssetIndex++;
			CurrentStage = EState::OpenEditor;
			WaitingForEditorCount = 0;

			if ( AssetIndex < Assets.Num() )
			{
				TestExecutionInfo->PushContext(Assets[AssetIndex].Asset->GetName());
			}
		}

		/** The asset list */
		TArray<FOpenAssetInfo> Assets;
		/** The current stage */
		EState::Type CurrentStage;
		/** The index of the current asset */
		int32 AssetIndex;
		/** How many ticks we have waited for the asset editor to open */
		int32 WaitingForEditorCount;
		/** The maximum number of ticks to wait for the editor*/
		int32 MaxWaitForEditorTicks;

		/** Pointer to the execution info to prefix logs*/
		FAutomationTestExecutionInfo* TestExecutionInfo;
	};

	/**
	* The main build promotion test class
	*/
	struct FBuildPromotionTest
	{
		/** Pointer to running automation test instance */
		FEditorPromotionTest* Test;

		/** Function definition for the test stage functions */
		typedef bool(BuildPromotionTestHelper::FBuildPromotionTest::*TestStageFn)();

		/** Pointer to the execution info to prefix logs*/
		FAutomationTestExecutionInfo* TestExecutionInfo;

		/** List of test stage functions */
		TArray<TestStageFn> TestStages;
		TArray<FString> StageNames;

		/** The index of the test stage we are on */
		int32 CurrentStage;

		/** Pointer to the active editor world */
		UWorld* CurrentWorld;

		/** Point light created by the test */
		APointLight* PointLight;

		/** If true, we will revert the auto apply lighting setting when the lighting build finishes */
		bool bDisableAutoApplyLighting;

		/** Items that were imported during the Import Workflow stage */
		UTexture* DiffuseTexture;
		UTexture* NormalTexture;
		UStaticMesh* WindowMesh;
		UStaticMesh* ReimportMesh;
		USkeletalMesh* BlendShape;
		USkeletalMesh* MorphAndMorphAnim;
		USkeletalMesh* SkeletalMesh_Test;
		UAnimSequence* AnimationTest;
		USoundWave* FemaleVoice;
		USoundWave* SurroundTest;

		/** Pointer to the material we are editing for the source control test stage */
		UMaterial* SCTestMat;
		FString ChosenMaterialColor;

		/** Particle System loaded from Automation Settings for Blueprint Pass */
		UParticleSystem* LoadedParticleSystem;

		/** Helper for opening, modifying, and placing assets */
		FOpenAssetHelper* OpenAssetHelper;

		/** Objects created by the Blueprint stages */
		UBlueprint* BlueprintObject;
		UPackage* BlueprintPackage;
		UEdGraphNode* PostBeginPlayEventNode;
		UEdGraphNode* PrintNode;
		AActor* PlacedBlueprint;

		/** Source control async helper */
		SourceControlAutomationCommon::FAsyncCommandHelper AsyncHelper;

		/** Delay helper */
		EditorBuildPromotionTestUtils::FDelayHelper DelayHelper;

		/** List of skipped tests */
		TArray<FString> SkippedTests;

		/** summary logs to display at the end */
		TArray<FString> SummaryLines;

		int32 SectionSuccessCount;
		int32 SectionTestCount;
		
#define ADD_TEST_STAGE(FuncName,StageName) \
	TestStages.Add(&BuildPromotionTestHelper::FBuildPromotionTest::FuncName); \
	StageNames.Add(StageName); 

		/**
		* Constructor
		*/
		FBuildPromotionTest(FAutomationTestExecutionInfo* InExecutionInfo) :
			CurrentStage(0)
		{
			FMemory::Memzero(this, sizeof(FBuildPromotionTest));

			TestExecutionInfo = InExecutionInfo;

			// 2) Geometry
			ADD_TEST_STAGE(Geometry_LevelCreationAndSetup,	TEXT("Level Creation and Setup"));
			ADD_TEST_STAGE(EndSection,						TEXT("Geometry Workflow"));

			// 3) Lighting
			ADD_TEST_STAGE(Lighting_BuildLighting_Part1,	TEXT("Build Lighting"));
			ADD_TEST_STAGE(Lighting_BuildLighting_Part2,	TEXT("Build Lighting"));
			ADD_TEST_STAGE(EndSection, TEXT("Lighting Workflow"));

			// 4) Importing Workflow
			ADD_TEST_STAGE(Workflow_ImportWorkflow, TEXT("")); // Not using a subsection name here because it would be redundant
			ADD_TEST_STAGE(EndSection, TEXT("Importing Workflow"));

			// 5) Content Browser
			ADD_TEST_STAGE(ContentBrowser_SourceControl_Part1,			TEXT("Revision Control"));
			ADD_TEST_STAGE(ContentBrowser_SourceControl_Part2,			TEXT("Revision Control"));
			ADD_TEST_STAGE(ContentBrowser_SourceControl_Part3,			TEXT("Revision Control"));
			ADD_TEST_STAGE(ContentBrowser_SourceControl_Part4,			TEXT("Revision Control"));
			ADD_TEST_STAGE(ContentBrowser_SourceControl_Part5,			TEXT("Revision Control"));
			ADD_TEST_STAGE(ContentBrowser_OpenAssets_Part1,				TEXT("Open Asset Types"));
			ADD_TEST_STAGE(ContentBrowser_OpenAssets_Part2,				TEXT("Open Asset Types"));
			ADD_TEST_STAGE(ContentBrowser_ReimportAsset,				TEXT("Re-import Assets"));
			ADD_TEST_STAGE(ContentBrowser_AssignAMaterial,				TEXT("Assigning a Material"));
			ADD_TEST_STAGE(EndSection, TEXT("Content Browser"));

			// 6) Blueprints
			ADD_TEST_STAGE(Blueprint_Setup,						TEXT("Blueprint setup"));
			ADD_TEST_STAGE(Blueprint_Placement_Part1,			TEXT("Blueprint Placement"));
			//ADD_TEST_STAGE(Blueprint_Placement_Part2,			TEXT("Blueprint Placement"));
			ADD_TEST_STAGE(Blueprint_Placement_Part3,			TEXT("Blueprint Placement"));
			ADD_TEST_STAGE(Blueprint_Placement_Part4,			TEXT("Blueprint Placement"));
			// Disabling breakpoint tests because they can't actually detect if the breakpoint is hit since we aren't ticked during intraframe debugging
			//ADD_TEST_STAGE(Blueprint_SetBreakpoint_Part1,		TEXT("Set Breakpoints"));
			//ADD_TEST_STAGE(Blueprint_SetBreakpoint_Part2,		TEXT("Set Breakpoints"));
			//ADD_TEST_STAGE(Blueprint_SetBreakpoint_Part3,		TEXT("Set Breakpoints"));
			ADD_TEST_STAGE(Blueprint_LevelScript_Part1,			TEXT("Level Script"));
			ADD_TEST_STAGE(Blueprint_LevelScript_Part2,			TEXT("Level Script"));
			ADD_TEST_STAGE(Blueprint_LevelScript_Part3,			TEXT("Level Script"));
			ADD_TEST_STAGE(Blueprint_LevelScript_Part4,			TEXT("Level Script"));
			ADD_TEST_STAGE(Blueprint_LevelScript_Part5,			TEXT("Level Script"));
			ADD_TEST_STAGE(EndSection, TEXT("Blueprint"));

			// 8) Build, play, save
			ADD_TEST_STAGE(Building_BuildLevel_Part1, TEXT("")); // Not using a subsection name here because it would be redundant
			ADD_TEST_STAGE(Building_BuildLevel_Part2, TEXT(""));
			ADD_TEST_STAGE(Building_BuildLevel_Part3, TEXT(""));
			ADD_TEST_STAGE(Building_BuildLevel_Part4, TEXT(""));
			ADD_TEST_STAGE(EndSection, TEXT("Building and Saving"));

			ADD_TEST_STAGE(LogSummary, TEXT(""));
		}
#undef ADD_TEST_STAGE

		/**
		* Runs the current test stage
		*/
		bool Update()
		{
			this->Test->PushContext(StageNames[CurrentStage]);
			bool bStageComplete = (this->*TestStages[CurrentStage])();
			this->Test->PopContext();

			if (bStageComplete)
			{
				CurrentStage++;
			}
			return CurrentStage >= TestStages.Num();
		}

	private:

		/**
		* Handle adding headers and success summary
		*/
		bool EndSection()
		{
			const FString SectionName = StageNames[CurrentStage];

			//Reset section test counts
			SectionSuccessCount = -1;
			SectionTestCount = -1;

			return true;
		}

		/**
		* Adds a summary to the end of the log
		*/
		bool LogSummary()
		{
			//Log out the summary lines
			if (SummaryLines.Num() > 0)
			{
				UE_LOGF(LogEditorBuildPromotionTests, Display, "\nSummary:");
				for (int32 i = 0; i < SummaryLines.Num(); ++i)
				{
					UE_LOGF(LogEditorBuildPromotionTests, Display, "    %ls", *SummaryLines[i]);
				}
			}
			
			//Log out skipped tests
			if (SkippedTests.Num() > 0)
			{
				UE_LOGF(LogEditorBuildPromotionTests, Display, "\nSkipped the following tests:");
				for (int32 i = 0; i < SkippedTests.Num(); ++i)
				{
					UE_LOGF(LogEditorBuildPromotionTests, Display, "    %ls", *SkippedTests[i]);
				}
			}

			UE_LOGF(LogEditorBuildPromotionTests, Display, "\nPlease restart the editor and continue to Step 2");
			return true;
		}

		/**
		* Geometry Test Stage: Level Creation and Setup
		*   Create a new map and add a light
		*/
		bool Geometry_LevelCreationAndSetup()
		{
			//Create a new level
			CurrentWorld = FAutomationEditorCommonUtils::CreateNewMap();
			UE_LOGF(LogEditorBuildPromotionTests, Display, "Created an empty level");

			//Add a directional light
			const FTransform Transform(FRotator(-45.f,5.f,0),FVector(0.0f, 0.0f, 400.0f));
			ADirectionalLight* DirectionalLight = Cast<ADirectionalLight>(GEditor->AddActor(CurrentWorld->GetCurrentLevel(), ADirectionalLight::StaticClass(), Transform));

			if (DirectionalLight)
			{
				UE_LOGF(LogEditorBuildPromotionTests, Display, "Placed a directional light");
			}
			else
			{
				UE_LOGF(LogEditorBuildPromotionTests, Error, "Failed to place directional light");
			}

			return true;
		}
		

		/**
		* Lighting Test Stage: Build Lighting (Part 1)
		*    Sets the lighting quality level and starts a lighting build
		*/
		bool Lighting_BuildLighting_Part1()
		{
			//Set production quality
			GConfig->SetInt(TEXT("LightingBuildOptions"), TEXT("QualityLevel"), (int32)ELightingBuildQuality::Quality_Production, GEditorPerProjectIni);
			UE_LOGF(LogEditorBuildPromotionTests, Display, "Set the lighting quality to Production");

			//Force AutoApplyLighting on
			ULevelEditorMiscSettings* LevelEdSettings = GetMutableDefault<ULevelEditorMiscSettings>();
			bDisableAutoApplyLighting = !LevelEdSettings->bAutoApplyLightingEnable;
			LevelEdSettings->bAutoApplyLightingEnable = true;

			//Build Lighting
			EditorBuildPromotionTestUtils::BuildLighting();

			return true;
		}

		/**
		* Lighting Test Stage: Build Lighting (Part 2)
		*    Waits for lighting to finish
		*/
		bool Lighting_BuildLighting_Part2()
		{
			if (!GUnrealEd->IsLightingBuildCurrentlyRunning())
			{
				if (bDisableAutoApplyLighting)
				{
					ULevelEditorMiscSettings* LevelEdSettings = GetMutableDefault<ULevelEditorMiscSettings>();
					LevelEdSettings->bAutoApplyLightingEnable = false;
				}
				UE_LOGF(LogEditorBuildPromotionTests, Display, "Built Lighting");
				return true;
			}
			return false;
		}

//@TODO: Rewrite this without macro if possible
#define IMPORT_ASSET_WITH_FACTORY(TFactoryClass,TObjectClass,ImportSetting,ClassVariable,ExtraSettings) \
{ \
	const FString FilePath = AutomationTestSettings->BuildPromotionTest.ImportWorkflow.ImportSetting.ImportFilePath.FilePath; \
	if (FilePath.Len() > 0) \
	{ \
		TFactoryClass* FactoryInst = NewObject<TFactoryClass>(); \
		ExtraSettings \
		FAutomationEditorCommonUtils::ApplyCustomFactorySettings(FactoryInst, AutomationTestSettings->BuildPromotionTest.ImportWorkflow.ImportSetting.FactorySettings); \
		ClassVariable = Cast<TObjectClass>(EditorBuildPromotionTestUtils::ImportAsset(FactoryInst, FilePath)); \
		EditorBuildPromotionTestUtils::PlaceImportedAsset(ClassVariable, PlaceLocation); \
	} \
	else \
	{ \
		SkippedTests.Add(FString::Printf(TEXT("Importing Workflow: Importing %s. (No file path)"),TEXT(#ImportSetting))); \
		UE_LOGF(LogEditorBuildPromotionTests, Log, "No asset import path set for %ls", TEXT(#ImportSetting)); \
	} \
}

		/**
		* Workflow Test Stage: Importing Workflow
		*    Imports a set of assets from the AutomationTestSettings and adds them to the map
		*/
		bool Workflow_ImportWorkflow()
		{
			UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
			check(AutomationTestSettings);

			FVector PlaceLocation(940, 810, 160);
			const float ActorSpacing = 200.f;

			//Diffuse
			IMPORT_ASSET_WITH_FACTORY(UTextureFactory, UTexture, Diffuse, DiffuseTexture, );

			//Normalmap
			IMPORT_ASSET_WITH_FACTORY(UTextureFactory, UTexture, Normal, NormalTexture, FactoryInst->LODGroup = TEXTUREGROUP_WorldNormalMap;);

			//Static Mesh
			IMPORT_ASSET_WITH_FACTORY(UFbxFactory, UStaticMesh, StaticMesh, WindowMesh, );

			//Reimport Static Mesh
			IMPORT_ASSET_WITH_FACTORY(UFbxFactory, UStaticMesh, ReimportStaticMesh, ReimportMesh, );

			//Blend Shape Mesh
			IMPORT_ASSET_WITH_FACTORY(UFbxFactory, USkeletalMesh, BlendShapeMesh, BlendShape, FactoryInst->ImportUI->MeshTypeToImport = FBXIT_SkeletalMesh;);

			//Morph Mesh
			IMPORT_ASSET_WITH_FACTORY(UFbxFactory, USkeletalMesh, MorphMesh, MorphAndMorphAnim, FactoryInst->ImportUI->MeshTypeToImport = FBXIT_SkeletalMesh;);

			//Skeletal Mesh
			IMPORT_ASSET_WITH_FACTORY(UFbxFactory, USkeletalMesh, SkeletalMesh, SkeletalMesh_Test, FactoryInst->ImportUI->MeshTypeToImport = FBXIT_SkeletalMesh;);

			if (SkeletalMesh_Test)
			{
				//Animation
				IMPORT_ASSET_WITH_FACTORY(UFbxFactory, UAnimSequence, Animation, AnimationTest, FactoryInst->ImportUI->MeshTypeToImport = FBXIT_Animation; FactoryInst->ImportUI->Skeleton = SkeletalMesh_Test->GetSkeleton(););
			}
			else
			{
				SkippedTests.Add(TEXT("Importing Workflow: Importing Animation.  (No skeletal mesh.)")); 
			}

			//Sound
			IMPORT_ASSET_WITH_FACTORY(USoundFactory, USoundWave, Sound, FemaleVoice, );

			//SurroundSound is a special case.  We need to import 6 files separately
			const FString SurroundFilePath = AutomationTestSettings->BuildPromotionTest.ImportWorkflow.SurroundSound.ImportFilePath.FilePath;
			if (SurroundFilePath.Len() > 0)
			{
				const FString BaseFileName = FPaths::GetPath(SurroundFilePath) / FPaths::GetBaseFilename(SurroundFilePath).LeftChop(3);

				USoundFactory* FactoryInst = NewObject<USoundFactory>();
				FAutomationEditorCommonUtils::ApplyCustomFactorySettings(FactoryInst, AutomationTestSettings->BuildPromotionTest.ImportWorkflow.SurroundSound.FactorySettings);

				const FString SurroundChannels[] = { TEXT("_fl"), TEXT("_fr"), TEXT("_fc"), TEXT("_lf"), TEXT("_sl"), TEXT("_sr"), TEXT("_bl"), TEXT("_br") };

				USoundWave* ImportedSound = NULL;
				for (int32 ChannelID = 0; ChannelID < UE_ARRAY_COUNT(SurroundChannels); ++ChannelID)
				{
					const FString ChannelFileName = FString::Printf(TEXT("%s%s.WAV"),*BaseFileName,*SurroundChannels[ChannelID]);
					if (FPaths::FileExists(ChannelFileName))
					{
						USoundWave* CreatedWave = Cast<USoundWave>(EditorBuildPromotionTestUtils::ImportAsset(FactoryInst, ChannelFileName));
						if (ImportedSound == NULL)
						{
							ImportedSound = CreatedWave;
						}
					}
				}

				if (ImportedSound)
				{
					EditorBuildPromotionTestUtils::PlaceImportedAsset(ImportedSound, PlaceLocation);
				}
				else
				{
					UE_LOGF(LogEditorBuildPromotionTests, Error, "Failed to create surround sound asset at (%ls)", *SurroundFilePath);
				}
			}
			else
			{
				SkippedTests.Add(TEXT("Importing Workflow: Importing SurroundSound. (No file path)"));
			}

			//Import the others
			const TArray<FEditorImportWorkflowDefinition>& AssetsToImport = AutomationTestSettings->BuildPromotionTest.ImportWorkflow.OtherAssetsToImport;
			for (int32 i = 0; i < AssetsToImport.Num(); ++i)
			{
				//Check the file path
				const FString FilePath = AssetsToImport[i].ImportFilePath.FilePath;
				if (FilePath.Len() > 0)
				{
					//Get the import factory class to use
					const FString FileExtension = FPaths::GetExtension(FilePath);
					UClass* FactoryClass = FAutomationEditorCommonUtils::GetFactoryClassForType(FileExtension);
					if (FactoryClass)
					{
						//Create the factory and import the asset
						UFactory* FactoryInst = NewObject<UFactory>(GetTransientPackage(), FactoryClass);
						FAutomationEditorCommonUtils::ApplyCustomFactorySettings(FactoryInst, AssetsToImport[i].FactorySettings);
						UObject* NewObject = EditorBuildPromotionTestUtils::ImportAsset(FactoryInst, FilePath);
						if (NewObject)
						{
							EditorBuildPromotionTestUtils::PlaceImportedAsset(NewObject, PlaceLocation);
						}
						else
						{
							UE_LOGF(LogEditorBuildPromotionTests, Warning, "Failed to create asset (%ls) with factory (%ls)", *FilePath, *FactoryClass->GetName());
						}
					}
					else
					{
						UE_LOGF(LogEditorBuildPromotionTests, Warning, "Couldn't find import factory to use on assset (%ls)", *FilePath);
					}
				} 
				else
				{
					UE_LOGF(LogEditorBuildPromotionTests, Log, "No asset import path set for OtherAssetsToImport.  Index: %i", i);
				}
			}

			//Remove one from the test counts to keep the section from counting
			SectionTestCount--;
			SectionSuccessCount--;

			//Save all the new assets
			EditorBuildPromotionTestUtils::SaveAllAssetsInFolder(FEditorPromotionTestUtilities::GetGamePath());

			return true;
		}
#undef IMPORT_ASSET_WITH_FACTORY

		/**
		* ContentBrowser Test Stage: Source Control (part 1)
		*    Opens the asset editor for the source control material
		*/
		bool ContentBrowser_SourceControl_Part1()
		{
			//Find the material to check out
			UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
			check(AutomationTestSettings);
			
			const FString SourceControlMaterialPath = AutomationTestSettings->BuildPromotionTest.SourceControlMaterial.FilePath;
			if (SourceControlMaterialPath.Len() > 0)
			{
				FAssetData MaterialData = FAutomationEditorCommonUtils::GetAssetDataFromPackagePath(SourceControlMaterialPath);
				SCTestMat = Cast<UMaterial>(MaterialData.GetAsset());

				if (SCTestMat)
				{
					//Open the asset editor
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(SCTestMat);
					UE_LOGF(LogEditorBuildPromotionTests, Display, "Opened the material editor for: %ls", *SCTestMat->GetName());

					FString PackageFileName = FPackageName::LongPackageNameToFilename(MaterialData.PackageName.ToString(), FPackageName::GetAssetPackageExtension());
					FString MaterialFilePath = FPaths::ConvertRelativePathToFull(PackageFileName);
					AsyncHelper = SourceControlAutomationCommon::FAsyncCommandHelper(MaterialFilePath);
				}
				else
				{
					UE_LOGF(LogEditorBuildPromotionTests, Error, "Failed to find material to modify for Content Browser tests.");
				}
			}
			else
			{
				SkippedTests.Add(TEXT("ContentBrowser: Revision Control. (No file path)"));
			}

			return true;
		}

		/**
		* ContentBrowser Test Stage: Source Control (part 2)
		*    Checks the source control material out of source control
		*/
		bool ContentBrowser_SourceControl_Part2()
		{			
			if (SCTestMat)
			{
				if (!AsyncHelper.IsDispatched())
				{
					if (ISourceControlModule::Get().GetProvider().Execute(
						ISourceControlOperation::Create<FCheckOut>(),
						SourceControlHelpers::PackageFilename(AsyncHelper.GetParameter()),
						EConcurrency::Asynchronous,
						FSourceControlOperationComplete::CreateRaw(&AsyncHelper, &SourceControlAutomationCommon::FAsyncCommandHelper::SourceControlOperationComplete)
						) != ECommandResult::Succeeded)
					{
						UE_LOGF(LogEditorBuildPromotionTests, Warning, "Failed to check out '%ls'", *AsyncHelper.GetParameter());
						return true;
					}

					AsyncHelper.SetDispatched();
				}

				if (AsyncHelper.IsDone())
				{
					// check state now we are done
					TSharedPtr<ISourceControlState, ESPMode::ThreadSafe> SourceControlState = ISourceControlModule::Get().GetProvider().GetState(SourceControlHelpers::PackageFilename(AsyncHelper.GetParameter()), EStateCacheUsage::Use);
					if (!SourceControlState.IsValid())
					{
						UE_LOGF(LogEditorBuildPromotionTests, Error, "Could not retrieve state for file '%ls'", *AsyncHelper.GetParameter());
					}
					else
					{
						if (!SourceControlState->IsCheckedOut())
						{
							UE_LOGF(LogEditorBuildPromotionTests, Error, "Unexpected state following Check Out operation for file '%ls'", *AsyncHelper.GetParameter());
						}
						else
						{
							UE_LOGF(LogEditorBuildPromotionTests, Display, "Checked out the revision control material");
						}
					}
				}

				return AsyncHelper.IsDone();
			}
			return true;
		}

		// @TODO: Rewrite this to use a lighter-weight asset type
		/**
		* ContentBrowser Test Stage: Source Control (part 3)
		*    Changes the source control material's color
		*/
		bool ContentBrowser_SourceControl_Part3()
		{
			if (SCTestMat)
			{
				UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
				check(AutomationTestSettings);

				IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(SCTestMat, true);
				FMaterialEditor* MaterialEditor = (FMaterialEditor*)AssetEditor;

				//Pick a random color
				const FString AvailableColors[] = { 
					TEXT("Red"),	TEXT("(R=1.0f,G=0.0f,B=0.0f)"), 
					TEXT("Green"),	TEXT("(R=0.0f,G=1.0f,B=0.0f)"), 
					TEXT("Blue"),	TEXT("(R=0.0f,G=0.0f,B=1.0f)"), 
					TEXT("Pink"),	TEXT("(R=1.0f,G=0.0f,B=1.0f)"),
					TEXT("Yellow"),	TEXT("(R=1.0f,G=1.0f,B=0.0f)"),
					TEXT("Black"),	TEXT("(R=0.0f,G=0.0f,B=0.0f)"),
					TEXT("White"),	TEXT("(R=1.0f,G=1.0f,B=1.0f)") };

				const int32 ChosenIndex = FMath::RandHelper(UE_ARRAY_COUNT(AvailableColors) / 2);
				ChosenMaterialColor = AvailableColors[ChosenIndex * 2];
				const FString ColorValue = AvailableColors[(ChosenIndex * 2)+1];

				UE_LOGF(LogEditorBuildPromotionTests, Display, "Changing revision control test to %ls", *ChosenMaterialColor);

				//Get the editor material
				UMaterial* EditorMaterial = Cast<UMaterial>(MaterialEditor->GetMaterialInterface());
				for (UMaterialExpression* Expression : EditorMaterial->GetExpressions())
				{
					UMaterialExpressionConstant3Vector* ColorParam = Cast<UMaterialExpressionConstant3Vector>(Expression);
					if (ColorParam)
					{
						EditorMaterial->Modify();
						FEditorPromotionTestUtilities::SetPropertyByName(ColorParam, TEXT("Constant"), ColorValue);
						MaterialEditor->UpdateMaterialAfterGraphChange();
						MaterialEditor->ForceRefreshExpressionPreviews();
						EditorBuildPromotionTestUtils::SendUpdateMaterialCommand();
					}
				}

				FAssetData MaterialData = FAutomationEditorCommonUtils::GetAssetDataFromPackagePath(AutomationTestSettings->BuildPromotionTest.SourceControlMaterial.FilePath);
				FString PackageFileName = FPackageName::LongPackageNameToFilename(MaterialData.PackageName.ToString(), FPackageName::GetAssetPackageExtension());
				FString MaterialFilePath = FPaths::ConvertRelativePathToFull(PackageFileName);
				AsyncHelper = SourceControlAutomationCommon::FAsyncCommandHelper(MaterialFilePath);
			}
			return true;
		}

		/**
		* ContentBrowser Test Stage: Source Control (part 4)
		*    Checks the source control material back in and sets the description to the new color
		*/
		bool ContentBrowser_SourceControl_Part4()
		{
			if (SCTestMat)
			{
				if (!AsyncHelper.IsDispatched())
				{
					TSharedRef<FCheckIn, ESPMode::ThreadSafe> CheckInOperation = ISourceControlOperation::Create<FCheckIn>();
					FString CheckinDescription = FString::Printf(TEXT("[AUTOMATED TEST] Changed Material Color to %s"), *ChosenMaterialColor);
					CheckInOperation->SetDescription(FText::FromString(CheckinDescription));

					if (ISourceControlModule::Get().GetProvider().Execute(
						CheckInOperation,
						SourceControlHelpers::PackageFilename(AsyncHelper.GetParameter()),
						EConcurrency::Asynchronous,
						FSourceControlOperationComplete::CreateRaw(&AsyncHelper, &SourceControlAutomationCommon::FAsyncCommandHelper::SourceControlOperationComplete)
						) != ECommandResult::Succeeded)
					{
						UE_LOGF(LogEditorBuildPromotionTests, Warning, "Failed to check in '%ls'", *AsyncHelper.GetParameter());
						return true;
					}

					AsyncHelper.SetDispatched();
				}

				if (AsyncHelper.IsDone())
				{
					// check state now we are done
					TSharedPtr<ISourceControlState, ESPMode::ThreadSafe> SourceControlState = ISourceControlModule::Get().GetProvider().GetState(SourceControlHelpers::PackageFilename(AsyncHelper.GetParameter()), EStateCacheUsage::Use);
					if (!SourceControlState.IsValid())
					{
						UE_LOGF(LogEditorBuildPromotionTests, Error, "Could not retrieve state for file '%ls'", *AsyncHelper.GetParameter());
					}
					else
					{
						if (!SourceControlState->IsSourceControlled() || !SourceControlState->CanCheckout())
						{
							UE_LOGF(LogEditorBuildPromotionTests, Error, "Unexpected state following Check In operation for file '%ls'", *AsyncHelper.GetParameter());
						}
						else
						{
							UE_LOGF(LogEditorBuildPromotionTests, Display, "Checked in the revision control material");
						}
					}
					return true;
				}
				return false;
			}
			return true;
		}

		/**
		* ContentBrowser Test Stage: Source Control (part 5)
		*    Closes the material editor
		*/
		bool ContentBrowser_SourceControl_Part5()
		{
			UE_LOGF(LogEditorBuildPromotionTests, Display, "Closed the material editor");
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllAssetEditors();
			return true;
		}

		/**
		* ContentBrowser Test Stage:  Open Assets (Part 1)
		*   Queues up assets to be open, modified, and placed into the level
		*/
		bool ContentBrowser_OpenAssets_Part1()
		{
			UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
			check(AutomationTestSettings);

			TArray<FOpenAssetInfo> OpenInfo;
			UObject* Asset = NULL;
			FAssetData AssetData;
			FString AssetPackagePath;

			// Blueprint
			AssetPackagePath = AutomationTestSettings->BuildPromotionTest.OpenAssets.BlueprintAsset.FilePath;
			if (AssetPackagePath.Len() > 0)
			{
				AssetData = FAutomationEditorCommonUtils::GetAssetDataFromPackagePath(AssetPackagePath);
				Asset = AssetData.GetAsset();
				if (Asset)
				{
					OpenInfo.Add(FOpenAssetInfo(Asset, AssetData, TEXT("BlueprintDescription"), TEXT("Modified by BuildPromotionTest TM")));
				}
				else
				{
					SkippedTests.Add(TEXT("ContentBrowser: Open Blueprint. (Asset not found)"));
					UE_LOGF(LogEditorBuildPromotionTests, Warning, "Skipping Asset: BlueprintAsset not found");
				}
			}
			else
			{
				SkippedTests.Add(TEXT("ContentBrowser: Open Blueprint. (No file path)"));
				UE_LOGF(LogEditorBuildPromotionTests, Warning, "Skipping Asset: BlueprintAsset setting is empty");
			}
			
			// Material
			AssetPackagePath = AutomationTestSettings->BuildPromotionTest.OpenAssets.MaterialAsset.FilePath;
			if (AssetPackagePath.Len() > 0)
			{
				AssetData = FAutomationEditorCommonUtils::GetAssetDataFromPackagePath(AssetPackagePath);
				Asset = AssetData.GetAsset();
				if (Asset)
				{
					OpenInfo.Add(FOpenAssetInfo(Asset, AssetData, TEXT("TwoSided"), TEXT("true")));
				}
				else
				{
					SkippedTests.Add(TEXT("ContentBrowser: Open Material. (Asset not found)"));
					UE_LOGF(LogEditorBuildPromotionTests, Warning, "Skipping Asset: MaterialAsset not found");
				}
			}
			else
			{
				SkippedTests.Add(TEXT("ContentBrowser: Open Material. (No file path)"));
				UE_LOGF(LogEditorBuildPromotionTests, Warning, "Skipping Asset: MaterialAsset setting is empty");
			}
			
			// Particle System
			AssetPackagePath = AutomationTestSettings->BuildPromotionTest.OpenAssets.ParticleSystemAsset.FilePath;  // @TODO: Use an Engine asset
			if (AssetPackagePath.Len() > 0)
			{
				AssetData = FAutomationEditorCommonUtils::GetAssetDataFromPackagePath(AssetPackagePath);
				Asset = AssetData.GetAsset();
				if (Asset)
				{
					OpenInfo.Add(FOpenAssetInfo(Asset, AssetData, TEXT("UpdateTime_FPS"), TEXT("100")));
				}
				else
				{
					SkippedTests.Add(TEXT("ContentBrowser: Open ParticleSystem. (Asset not found)"));
					UE_LOGF(LogEditorBuildPromotionTests, Warning, "Skipping Asset: ParticleSystemAsset not found");
				}
			}
			else
			{
				SkippedTests.Add(TEXT("ContentBrowser: Open ParticleSystem. (No file path)"));
				UE_LOGF(LogEditorBuildPromotionTests, Warning, "Skipping Asset: ParticleSystemAsset setting is empty");
			}

			// Skeletal Mesh
			AssetPackagePath = AutomationTestSettings->BuildPromotionTest.OpenAssets.SkeletalMeshAsset.FilePath;
			if (AssetPackagePath.Len() > 0)
			{
				AssetData = FAutomationEditorCommonUtils::GetAssetDataFromPackagePath(AssetPackagePath);
				Asset = AssetData.GetAsset();
				if (Asset)
				{
					OpenInfo.Add(FOpenAssetInfo(Asset, AssetData, TEXT("bUseFullPrecisionUVs"), TEXT("1")));
				}
				else
				{
					SkippedTests.Add(TEXT("ContentBrowser: Open SkeletalMesh. (Asset not found)"));
					UE_LOGF(LogEditorBuildPromotionTests, Warning, "Skipping Asset: SkeletalMeshAsset not found");
				}
			}
			else
			{
				SkippedTests.Add(TEXT("ContentBrowser: Open SkeletalMesh. (No file path)"));
				UE_LOGF(LogEditorBuildPromotionTests, Warning, "Skipping Asset: SkeletalMeshAsset setting is empty");
			}
			
			// Static Mesh
			AssetPackagePath = AutomationTestSettings->BuildPromotionTest.OpenAssets.StaticMeshAsset.FilePath;
			if (AssetPackagePath.Len() > 0)
			{
				AssetData = FAutomationEditorCommonUtils::GetAssetDataFromPackagePath(AssetPackagePath);
				Asset = AssetData.GetAsset();
				if (Asset)
				{
					OpenInfo.Add(FOpenAssetInfo(Asset, AssetData, TEXT("AutoLODPixelError"), TEXT("42.f")));
				}
				else
				{
					SkippedTests.Add(TEXT("ContentBrowser: Open StaticMesh. (Asset not found)"));
					UE_LOGF(LogEditorBuildPromotionTests, Warning, "Skipping Asset: StaticMeshAsset not found");
				}
			}
			else
			{
				SkippedTests.Add(TEXT("ContentBrowser: Open StaticMesh. (No file path)"));
				UE_LOGF(LogEditorBuildPromotionTests, Warning, "Skipping Asset: StaticMeshAsset setting is empty");
			}
			
			// Texture
			AssetPackagePath = AutomationTestSettings->BuildPromotionTest.OpenAssets.TextureAsset.FilePath;
			if (AssetPackagePath.Len() > 0)
			{
				AssetData = FAutomationEditorCommonUtils::GetAssetDataFromPackagePath(AssetPackagePath);
				Asset = AssetData.GetAsset();
				if (Asset)
				{
					OpenInfo.Add(FOpenAssetInfo(Asset, AssetData, TEXT("LODBias"), TEXT("2")));
				}
				else
				{
					SkippedTests.Add(TEXT("ContentBrowser: Open Texture. (Asset not found)"));
					UE_LOGF(LogEditorBuildPromotionTests, Warning, "Skipping Asset: TextureAsset not found");
				}
			}
			else
			{
				SkippedTests.Add(TEXT("ContentBrowser: Open Texture. (No file path)"));
				UE_LOGF(LogEditorBuildPromotionTests, Warning, "Skipping Asset: TextureAsset setting is empty");
			}
			
			OpenAssetHelper = new FOpenAssetHelper(OpenInfo, TestExecutionInfo);

			return true;
		}

		/**
		* ContentBrowser Test Stage: Open Assets (Part 2)
		*    Waits for the OpenAssetHeler to finish 
		*/
		bool ContentBrowser_OpenAssets_Part2()
		{
			if (OpenAssetHelper)
			{
				if (OpenAssetHelper->Update())
				{
					delete OpenAssetHelper;
					OpenAssetHelper = NULL;
					return true;
				}
				return false;
			}
			else
			{
				return true;
			}
		}

		/**
		* ContentBrowser Test Stage: Reimport Asset
		*    Reimports the static mesh
		*/
		bool ContentBrowser_ReimportAsset()
		{
			if (ReimportMesh)
			{
				if (FReimportManager::Instance()->Reimport(ReimportMesh, /*bAskForNewFileIfMissing=*/false))
				{
					UE_LOGF(LogEditorBuildPromotionTests, Display, "Reimported asset %ls", *ReimportMesh->GetName());
				}
				else
				{
					UE_LOGF(LogEditorBuildPromotionTests, Error, "Failed to reimport asset %ls", *ReimportMesh->GetName());
				}
			}
			else
			{
				SkippedTests.Add(TEXT("ContentBrowser: Reimport Asset.  (No Reimport mesh)"));
			}

			return true;
		}

		/**
		* ContentBrowser Test Stage: Creating a material (Part 3)
		*    Closes all assets editor and adds the material to a default object in the map
		*/
		bool ContentBrowser_AssignAMaterial()
		{
			// SETUP
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllAssetEditors();
			UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
			check(AutomationTestSettings);

			// Load default material asset
			FString MaterialPackagePath = AutomationTestSettings->MaterialEditorPromotionTest.DefaultMaterialAsset.FilePath;
			if (!(MaterialPackagePath.Len() > 0))
			{
				UE_LOGF(LogEditorBuildPromotionTests, Warning, "Skipping material assignment test: no default material defined.");
				return true;
			}

			FAssetData MaterialAssetData = FAutomationEditorCommonUtils::GetAssetDataFromPackagePath(MaterialPackagePath);
			UMaterial* DefaultMaterial = Cast<UMaterial>(MaterialAssetData.GetAsset());
			if (!DefaultMaterial)
			{
				UE_LOGF(LogEditorBuildPromotionTests, Error, "Failed to load default material asset.");
				return false;
			}

			// Add static mesh to world as material assignment target
			const FVector PlaceLocation(0, 2240, 166);
			AStaticMeshActor* PlacedStaticMesh = EditorBuildPromotionTestUtils::AddDefaultMeshToLevel(PlaceLocation);

			// RUN TEST
			// @TODO: Put in a check to verify that the mesh's Material[0] == DefaultMaterial
			if (EditorBuildPromotionTestUtils::ApplyMaterialToStaticMesh(PlacedStaticMesh, DefaultMaterial))
			{
				UE_LOGF(LogEditorBuildPromotionTests, Display, "Successfully assigned material to static mesh.");
			}  // No need to error on false, since ApplyMaterialToStaticMesh triggers its own error if it fails

			// @TODO: TEARDOWN

			return true;
		}

		/**
		* Saves the blueprint stored in BlueprintObject
		*/
		void SaveBlueprint()
		{
			if (BlueprintObject && BlueprintPackage)
			{
				BlueprintPackage->SetDirtyFlag(true);
				BlueprintPackage->FullyLoad();
				const FString PackagePath = FEditorPromotionTestUtilities::GetGamePath() + TEXT("/") + EditorBuildPromotionTestUtils::BlueprintNameString;
				FSavePackageArgs SaveArgs;
				SaveArgs.TopLevelFlags = RF_Standalone;
				SaveArgs.Error = GLog;
				bool bHasPackageSaved = UPackage::SavePackage(BlueprintPackage, nullptr,
					*FPackageName::LongPackageNameToFilename(PackagePath, FPackageName::GetAssetPackageExtension()),
					SaveArgs);
				Test->TestTrue(FString::Printf(TEXT("Saved blueprint (%s)"), *BlueprintObject->GetName()), bHasPackageSaved);
			}
		}

		bool Blueprint_Setup()
		{
			const FString PackageName = FEditorPromotionTestUtilities::GetGamePath() + TEXT("/") + EditorBuildPromotionTestUtils::BlueprintNameString;
			
			// Create blueprint asset
			UBlueprintFactory* Factory = NewObject<UBlueprintFactory>();
			Factory->ParentClass = AActor::StaticClass();
			BlueprintPackage = CreatePackage(*PackageName);
			EObjectFlags Flags = RF_Public | RF_Standalone;

			// Check that conflicting asset doesn't already exist
			UObject* ExistingBlueprint = FindObject<UBlueprint>(BlueprintPackage, *EditorBuildPromotionTestUtils::BlueprintNameString);
			Test->TestNull(TEXT("Blueprint asset does not already exist (delete blueprint and restart editor)"), ExistingBlueprint);
			if (ExistingBlueprint)
			{
				return true;
			}

			//Save blueprint object for reuse in later stages
			BlueprintObject = Cast<UBlueprint>(Factory->FactoryCreateNew(UBlueprint::StaticClass(), BlueprintPackage, FName(*EditorBuildPromotionTestUtils::BlueprintNameString), Flags, NULL, GWarn));
			Test->TestNotNull(TEXT("Blueprint test asset created"), BlueprintObject);
			if (!BlueprintObject)
			{
				return true;
			}
			// Add asset to registry
			FAssetRegistryModule::AssetCreated(BlueprintObject);
			BlueprintPackage->MarkPackageDirty();

			//Add BeginPlay event to graph
			UEdGraph* EventGraph = FBlueprintEditorUtils::FindEventGraph(BlueprintObject);
			PostBeginPlayEventNode = EditorBuildPromotionTestUtils::CreatePostBeginPlayEvent(BlueprintObject, EventGraph);
			Test->TestNotNull(TEXT("Event Being Play node added"), PostBeginPlayEventNode);
			if (!PostBeginPlayEventNode)
			{
				return true;
			}

			// Add string member variable
			FEdGraphPinType StringPinType(UEdGraphSchema_K2::PC_String, NAME_None, nullptr, EPinContainerType::None, false, FEdGraphTerminalType());
			FBlueprintEditorUtils::AddMemberVariable(BlueprintObject, EditorBuildPromotionTestUtils::BlueprintStringVariableName, StringPinType);

			// Add print string node connected to the BeginPlay node; save it for use later
			UEdGraphPin* PlayThenPin = PostBeginPlayEventNode->FindPin(UEdGraphSchema_K2::PN_Then);
			PrintNode = EditorBuildPromotionTestUtils::AddPrintStringNode(BlueprintObject, EventGraph, PlayThenPin);
			Test->TestNotNull(TEXT("Print String node added"), PrintNode);
			
			return true;

		}

		/**
		* Blueprint Test Stage: Blueprint placement (Part 1)
		*    Places the blueprint in the level then starts PIE
		*/
		bool Blueprint_Placement_Part1()
		{
			if (BlueprintObject)
			{
				PlacedBlueprint = FActorFactoryAssetProxy::AddActorForAsset(BlueprintObject);
				Test->TestNotNull(TEXT("Blueprint instance placed in world"), PlacedBlueprint);
				if (PlacedBlueprint)
				{
					//Set the text
					const FString NewVariableText = TEXT("Print String works!");
					FEditorPromotionTestUtilities::SetPropertyByName(PlacedBlueprint, EditorBuildPromotionTestUtils::BlueprintStringVariableName.ToString(), NewVariableText);
					// TODO: Validate that property updated correctly. Needs to be done in latent command?
					Test->AddInfo(TEXT("Updated string variable value"));
				}

				GEditor->SelectNone(false, true);

				//Start PIE
				EditorBuildPromotionTestUtils::StartPIE(true);

				//Make sure the timer is reset
				DelayHelper.Reset();
			}
			return true;
		}

		///**
		//* Blueprint Test Stage: Blueprint placement (Part 2)
		//*    Takes a screenshot of the initial state of the blueprint
		//*/
		//bool Blueprint_Placement_Part2()
		//{
		//	if (BlueprintObject && PlacedBlueprint)
		//	{
		//		FEditorPromotionTestUtilities::TakeScreenshot(TEXT("BlueprintPIE_Start"));
		//	}
		//	return true;
		//}

		/**
		* Blueprint Test Stage: Blueprint placement (Part 3)
		*    Waits for 2 seconds for the timer to finish
		*/
		bool Blueprint_Placement_Part3()
		{
			if (BlueprintObject && PlacedBlueprint)
			{
				//Set a timeout in case PIE doesn't work
				if (!DelayHelper.IsRunning())
				{
					DelayHelper.Start(5.0f);
				}
				else if (DelayHelper.IsComplete())
				{
					//FAILED to hit breakpoint in time
					Test->AddError(TEXT("Timed out waiting for PIE to start"));
					DelayHelper.Reset();
					return true;
				}

				//Wait for PIE to startup
				if (GEditor->PlayWorld != NULL)
				{
					//Stop after 2 seconds of gameplay
					if (GEditor->PlayWorld->TimeSeconds > 2.0f)
					{
						DelayHelper.Reset();
						return true;
					}
				}
				return false;
			}
			return true;
		}

		/**
		* Blueprint Test Stage: Blueprint placement (Part 4)
		*    Takes a screenshot after the blueprint has changed.  Ends the PIE session.
		*/
		bool Blueprint_Placement_Part4()
		{
			if (BlueprintObject && PlacedBlueprint)
			{
				//if (GEditor->PlayWorld != NULL)
				//{
				//	FEditorPromotionTestUtilities::TakeScreenshot(TEXT("BlueprintPIE_End"));
				//}
				FEditorPromotionTestUtilities::EndPIE();
			}
			return true;
		}

		/**
		* Blueprint Test Stage: Set Breakpoint (Part 1)
		*    Sets a breakpoint on the PrintString node and starts PIE
		*/
		bool Blueprint_SetBreakpoint_Part1()
		{
			if (BlueprintObject)
			{
				//Add a breakpoint
				FKismetDebugUtilities::CreateBreakpoint(BlueprintObject, PrintNode, /* bIsEnabled = */ true);

				Test->AddInfo(TEXT("Set a breakpoint on the PrintString node"));
				EditorBuildPromotionTestUtils::StartPIE(true);

				//Make sure the timer is reset
				DelayHelper.Reset();
			}
			return true;
		}

		/**
		* Blueprint Test Stage: Set Breakpoint (Part 2)
		*    Waits for the breakpoint to get hit or a 10 second timeout to expire
		*/
		bool Blueprint_SetBreakpoint_Part2()
		{
			if (BlueprintObject)
			{
				//Set a timeout for hitting the breakpoint
				if (!DelayHelper.IsRunning())
				{
					DelayHelper.Start(10.0f);
				}
				else if (DelayHelper.IsComplete())
				{
					//FAILED to hit breakpoint in time
					Test->AddError(TEXT("Failed to hit the breakpoint after 10 seconds"));
					DelayHelper.Reset();
					return true;
				}

				UEdGraphNode* CurrentBreakpointNode = NULL;
				{
					//Hack.  GetMostRecentBreakpointHit only returns data if GIntraFrameDebuggingGameThread is true.
					TGuardValue<bool> SignalGameThreadBeingDebugged(GIntraFrameDebuggingGameThread, true);
					CurrentBreakpointNode = FKismetDebugUtilities::GetMostRecentBreakpointHit();
				}

				//Wait for breakpoint to get hit
				if (CurrentBreakpointNode == PrintNode)
				{
					//Success!
					Test->AddInfo(TEXT("Hit the PrintString breakpoint"));
					DelayHelper.Reset();
					return true;
				}
				return false;
			}
			return true;
		}

		/**
		* Blueprint Test Stage: Set Breakpoint (Part 3)
		*    Ends the PIE session and clears the breakpoint
		*/
		bool Blueprint_SetBreakpoint_Part3()
		{
			if (BlueprintObject)
			{
				Test->AddInfo(TEXT("Clearing the breakpoint"));
				FKismetDebugUtilities::ClearBreakpoints(BlueprintObject);
				FEditorPromotionTestUtilities::EndPIE();
				CurrentWorld = PlacedBlueprint->GetWorld();
			}
			return true;
		}

		/**
		* Blueprint Test Stage: Level Script (Part 1)
		*    Finds and opens the level script blueprint
		*/
		bool Blueprint_LevelScript_Part1()
		{
			if (BlueprintObject)
			{
				//Open the level script blueprint
				ULevelScriptBlueprint* LSB = PlacedBlueprint->GetLevel()->GetLevelScriptBlueprint(false);
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(LSB);
				Test->AddInfo(TEXT("Opened the level script blueprint"));
			}
			return true;
		}

		/**
		* Blueprint Test Stage: Level Script (Part 2)
		*    Copies the event nodes from the blueprint to the level script and compiles.  
		*    Removes the variables and function from the level script and compiles again.
		*    Modifies the Delay and PrintString values
		*    Starts a PIE session
		*/
		bool Blueprint_LevelScript_Part2()
		{
			if (BlueprintObject)
			{
				IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(BlueprintObject, true);
				FBlueprintEditor* BlueprintEditor = (FBlueprintEditor*)AssetEditor;

				UEdGraph* EventGraph = FBlueprintEditorUtils::FindEventGraph(BlueprintObject);
				TSet<UObject*> NodesToExport;
				for (int32 i = 0; i < EventGraph->Nodes.Num(); ++i)
				{
					EventGraph->Nodes[i]->PrepareForCopying();
					NodesToExport.Add(EventGraph->Nodes[i]);
				}

				ULevelScriptBlueprint* LSB = PlacedBlueprint->GetLevel()->GetLevelScriptBlueprint(true);

				{
					FString OutNodeText;
					FEdGraphUtilities::ExportNodesToText(NodesToExport, OutNodeText);
					FPlatformApplicationMisc::ClipboardCopy(*OutNodeText);

					UEdGraph* LevelEventGraph = FBlueprintEditorUtils::FindEventGraph(LSB);
					FKismetEditorUtilities::PasteNodesHere(LevelEventGraph, FVector2D(0, 0));

					//Note: These are a little out of order because logs are disabled above
					Test->AddInfo(TEXT("Copied the blueprint event nodes"));
					Test->AddInfo(TEXT("Pasted the nodes into to level script"));
				}

				//Compile the blueprint
				EditorBuildPromotionTestUtils::CompileBlueprint(LSB);

				//Test PIE
				EditorBuildPromotionTestUtils::StartPIE(true);

				//Make sure the timer is reset
				DelayHelper.Reset();
			}
			return true;
		}

		/**
		* Blueprint Test Stage: Level Script (Part 3)
		*    Waits for the delay timer in the level script
		*/
		bool Blueprint_LevelScript_Part3()
		{
			if (BlueprintObject)
			{
				//Set a timeout in case PIE doesn't start
				if (!DelayHelper.IsRunning())
				{
					DelayHelper.Start(8.0f);
				}
				else if (DelayHelper.IsComplete())
				{
					//FAILED to hit breakpoint in time
					Test->AddError(TEXT("Timed out waiting for PIE to start"));
					DelayHelper.Reset();
					return true;
				}

				//Wait for PIE to startup
				if (GEditor->PlayWorld != NULL)
				{
					//Stop after 4 seconds of gameplay
					if (GEditor->PlayWorld->TimeSeconds > 4.0f)
					{
						DelayHelper.Reset();
						return true;
					}
				}
				return false;
			}
			return true;
		}

		/**
		* Blueprint Test Stage: Level Script (Part 4)
		*    Takes a screenshot and ends PIE
		*/
		bool Blueprint_LevelScript_Part4()
		{
			if (BlueprintObject)
			{
				//if (GEditor->PlayWorld != NULL)
				//{
				//	//Take a screenshot and end the PIE session
				//	FEditorPromotionTestUtilities::TakeScreenshot(TEXT("LevelBlueprint"), false);
				//}
				FEditorPromotionTestUtilities::EndPIE();
			}
			return true;
		}

		/**
		* Blueprint Test Stage: Level Script (Part 5)
		*    Closes the blueprint editor and saves the blueprint
		*/
		bool Blueprint_LevelScript_Part5()
		{
			if (BlueprintObject)
			{
				Test->AddInfo(TEXT("Closing the blueprint editor"));
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllAssetEditors();
				//UE_LOGF(LogEditorBuildPromotionTests, Display, "Saving the blueprint");
				//SaveBlueprint();
			}
			return true;
		}

		/**
		* Building Test Stage: Building and Saving (Part 1)
		*    Toggles the level visibility off
		*/
		bool Building_BuildLevel_Part1()
		{
			UWorld* World = GEditor->GetEditorWorldContext().World();
			ULevel* Level = World->GetCurrentLevel();

			//Save all the new assets
			EditorBuildPromotionTestUtils::SaveAllAssetsInFolder(FEditorPromotionTestUtilities::GetGamePath());

			UE_LOGF(LogEditorBuildPromotionTests, Display, "Turning level visibility off");
			bool bShouldBeVisible = false;
			EditorLevelUtils::SetLevelVisibility(Level, bShouldBeVisible, false);
			GEditor->RedrawAllViewports(true);

			return true;
		}

		/**
		* Building Test Stage:  Building and Saving (Part 2)
		*   Takes a screenshot and toggles the level visibility back on
		*/
		bool  Building_BuildLevel_Part2()
		{
			UWorld* World = GEditor->GetEditorWorldContext().World();
			ULevel* Level = World->GetCurrentLevel();

			UE_LOGF(LogEditorBuildPromotionTests, Display, "Taking a screenshot");
			FEditorPromotionTestUtilities::TakeScreenshot(TEXT("VisibilityOff"), FAutomationScreenshotOptions(EComparisonTolerance::Low));

			UE_LOGF(LogEditorBuildPromotionTests, Display, "Turning level visibility on");
			bool bShouldBeVisible = true;
			EditorLevelUtils::SetLevelVisibility(Level, bShouldBeVisible, false);

			return true;
		}

		/**
		* Building Test Stage:  Building and Saving (Part 3)
		*   Takes a screenshot and does a full level rebuild
		*/
		bool  Building_BuildLevel_Part3()
		{
			UWorld* World = GEditor->GetEditorWorldContext().World();
			ULevel* Level = World->GetCurrentLevel();

			UE_LOGF(LogEditorBuildPromotionTests, Display, "Taking a screenshot");
			FEditorPromotionTestUtilities::TakeScreenshot(TEXT("VisibilityOn"), FAutomationScreenshotOptions(EComparisonTolerance::Low));

			FEditorFileUtils::SaveLevel(Level, TEXT("/Game/Maps/EditorBuildPromotionTest"));
			GUnrealEd->Exec(World, TEXT("MAP REBUILD ALLVISIBLE"));
			UE_LOGF(LogEditorBuildPromotionTests, Display, "Rebuilt the map");

			if (World->GetNavigationSystem())
			{
				// Invoke navmesh generator
				FNavigationSystem::Build(*World);
				UE_LOGF(LogEditorBuildPromotionTests, Display, "Built navigation");
			}

			//Force AutoApplyLighting on
			ULevelEditorMiscSettings* LevelEdSettings = GetMutableDefault<ULevelEditorMiscSettings>();
			bDisableAutoApplyLighting = !LevelEdSettings->bAutoApplyLightingEnable;
			LevelEdSettings->bAutoApplyLightingEnable = true;

			//Build Lighting
			EditorBuildPromotionTestUtils::BuildLighting();

			return true;
		}

		/**
		* Building Test Stage: Building and Saving (Part 4)
		*    Waits for the lighting to finish building and saves the level
		*/
		bool Building_BuildLevel_Part4()
		{
			if (!GUnrealEd->IsLightingBuildCurrentlyRunning())
			{
				UE_LOGF(LogEditorBuildPromotionTests, Display, "Built Lighting");

				if (bDisableAutoApplyLighting)
				{
					ULevelEditorMiscSettings* LevelEdSettings = GetMutableDefault<ULevelEditorMiscSettings>();
					LevelEdSettings->bAutoApplyLightingEnable = false;
				}

				UE_LOGF(LogEditorBuildPromotionTests, Display, "Saved the Level (EditorBuildPromotionTest)");

				UWorld* World = GEditor->GetEditorWorldContext().World();
				ULevel* Level = World->GetCurrentLevel();
				FEditorFileUtils::SaveLevel(Level);

				//Save all the new assets again because material usage flags may have changed.
				EditorBuildPromotionTestUtils::SaveAllAssetsInFolder(FEditorPromotionTestUtilities::GetGamePath());

				return true;
			}
			return false;
		}
	};
}

/**
* Automation test that handles cleanup of the build promotion test
*/
bool FBuildPromotionInitialCleanupTest::RunTest(const FString& Parameters)
{
	EditorBuildPromotionTestUtils::PerformCleanup();
	return true;
}


/**
* Latent command to run the main build promotion test
*/
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FRunBuildPromotionTestCommand, TSharedPtr<BuildPromotionTestHelper::FBuildPromotionTest>, BuildPromotionTest);
bool FRunBuildPromotionTestCommand::Update()
{
	return BuildPromotionTest->Update();
}

/**
* Automation test that handles the build promotion process
*/
bool FEditorPromotionTest::RunTest(const FString& Parameters)
{
	TSharedPtr<BuildPromotionTestHelper::FBuildPromotionTest> BuildPromotionTest = MakeShareable(new BuildPromotionTestHelper::FBuildPromotionTest(&ExecutionInfo));
	BuildPromotionTest->Test = this;
	ADD_LATENT_AUTOMATION_COMMAND(FRunBuildPromotionTestCommand(BuildPromotionTest));
	return true;
}

/**
* Latent command to run the main build promotion test
*/
DEFINE_LATENT_AUTOMATION_COMMAND(FEndPIECommand);
bool FEndPIECommand::Update()
{
	UE_LOGF(LogEditorBuildPromotionTests, Display, "Ending PIE");
	FEditorPromotionTestUtilities::EndPIE();
	return true;
}

/**
* Latent command to run the main build promotion test
*/
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FPIEExecCommand, FString, ExecCommand);
bool FPIEExecCommand::Update()
{
	if (GEditor->PlayWorld != NULL)
	{
		GEngine->Exec(GEditor->PlayWorld, *ExecCommand);
	}
	else
	{
		UE_LOGF(LogEditorBuildPromotionTests, Error, "Tried to execute a PIE command but PIE is not running. (%ls)", *ExecCommand);
	}
	return true;
}

/**
* Execute the loading of one map to verify PIE works
*/
bool FBuildPromotionPIETest::RunTest(const FString& Parameters)
{
	UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
	check(AutomationTestSettings);

	bool bLoadAsTemplate = false;
	bool bShowProgress = false;
	const FString MapName = FPaths::ProjectContentDir() + TEXT("Maps/EditorBuildPromotionTest.umap");
	FEditorFileUtils::LoadMap(MapName, bLoadAsTemplate, bShowProgress);
	UE_LOGF(LogEditorBuildPromotionTests, Display, "Loading Map: %ls", *MapName);

	EditorBuildPromotionTestUtils::StartPIE(false);
	UE_LOGF(LogEditorBuildPromotionTests, Display, "Starting PIE");

	//Find the main editor window
	TArray<TSharedRef<SWindow> > AllWindows;
	FSlateApplication::Get().GetAllVisibleWindowsOrdered(AllWindows);
	if (AllWindows.Num() == 0)
	{
		UE_LOGF(LogEditorAutomationTests, Error, "ERROR: Could not find the main editor window.");
		UE_LOGF(LogEditorAutomationTests, Display, "Test FAILED");
		return true;
	}

	WindowScreenshotParameters ScreenshotParams;
	ScreenshotParams.ScreenshotName = AutomationCommon::GetScreenshotPath(TEXT("EditorBuildPromotion/RunMap"));
	ScreenshotParams.CurrentWindow = AllWindows[0];
	//Wait for the play world to come up
	ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(1.f));

	//Toggle a stat and take a screenshot
	UE_LOGF(LogEditorBuildPromotionTests, Display, "Toggling \"Stat Memory\" and taking a screenshot");
	ADD_LATENT_AUTOMATION_COMMAND(FPIEExecCommand(TEXT("STAT Memory")));
	//Stat memory doesn't update right away so wait another second.
	ADD_LATENT_AUTOMATION_COMMAND(FPIEExecCommand(TEXT("STAT None")));
	ADD_LATENT_AUTOMATION_COMMAND(FEndPIECommand());

	return true;
}

/**
* Automation test that handles cleanup of the build promotion test
*/
bool FBuildPromotionCleanupTest::RunTest(const FString& Parameters)
{
	EditorBuildPromotionTestUtils::PerformCleanup();
	return true;
}

#undef LOCTEXT_NAMESPACE
