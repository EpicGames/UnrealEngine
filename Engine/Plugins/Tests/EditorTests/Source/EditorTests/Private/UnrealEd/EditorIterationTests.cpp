// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/Paths.h"
#include "Misc/AutomationTest.h"
#include "Misc/App.h"
#include "UObject/Object.h"

#include "Tests/AutomationTestSettings.h"
#include "Tests/AutomationEditorCommon.h"
#include "Tests/AutomationCommon.h"
#include "AutomationControllerSettings.h"
#include "AssetCompilingManager.h"

#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"

static TAutoConsoleVariable<int32> CVarProjectIterationPIETestIterations(
	TEXT("project.iteration.pie.testiterations"),
	1,
	TEXT("Number of iterations of PIE to run during the Project.Iteration.PIE test.")
);

/**
 * Test to open the sub editor windows for a specified list of assets.
 * This list can be setup in the Editor Preferences window within the editor or the DefaultEngine.ini file for that particular project.
*/
IMPLEMENT_CUSTOM_COMPLEX_AUTOMATION_TEST(FIterationOpenAssets, FAutomationTestBase, "Project.Iteration.PIE", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter);

void FIterationOpenAssets::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	const UAutomationTestSettings* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
	for ( FSoftObjectPath AssetRef : AutomationTestSettings->MapsToPIETest)
	{
		OutBeautifiedNames.Add(AssetRef.GetAssetName());
		OutTestCommands.Add(AssetRef.GetLongPackageName());
	}
}

bool FIterationOpenAssets::RunTest(const FString& LongAssetPath)
{
	static uint64 FrameNumber = 0;
	const uint64 NumFramesPerIteration = 600;
	const uint64 NumFramesBetweenIterations = 60;

	check(LongAssetPath.Len());

	// Setup
	AddCommand(new FCloseAllAssetEditorsCommand());
	AddCommand(new FFunctionLatentCommand([LongAssetPath] {
		TRACE_BOOKMARK(TEXT("LoadAsset - %s"), *LongAssetPath);
		return true;
		}));

	// Issue Load request
	AddCommand(new FOpenEditorForAssetCommand(LongAssetPath));

	AddCommand(new FFunctionLatentCommand([LongAssetPath] {
		TRACE_BOOKMARK(TEXT("LoadAssetComplete - %s"), *LongAssetPath);
		return true;
		}));

	int32 IterationNumber = 1;
	int32 TotalIterations = CVarProjectIterationPIETestIterations.GetValueOnAnyThread();
	for (; IterationNumber <= TotalIterations; IterationNumber++)
	{
		AddCommand(new FFunctionLatentCommand([LongAssetPath, IterationNumber] {
			TRACE_BOOKMARK(TEXT("PIE - %s - Iteration %d"), *LongAssetPath, IterationNumber);
			return true;
			}));

		// Do many frames of PIE (not a time span since PIE can take a while to init)
		AddCommand(new FDelayedFunctionLatentCommand([] {
			FrameNumber = GFrameCounter;
			}));
		AddCommand(new FStartPIECommand(false));
		AddCommand(new FFunctionLatentCommand([LongAssetPath] {
			return GFrameCounter > (FrameNumber + NumFramesPerIteration);
			}));
		AddCommand(new FEndPlayMapCommand());

		// Teardown
		AddCommand(new FFunctionLatentCommand([LongAssetPath, IterationNumber] {
			TRACE_BOOKMARK(TEXT("PIEComplete - %s - Iteration %d"), *LongAssetPath, IterationNumber);
			return true;
			}));

		if (IterationNumber < TotalIterations)
		{
			// Pause between iterations
			AddCommand(new FFunctionLatentCommand([LongAssetPath] {
				return GFrameCounter > (FrameNumber + NumFramesPerIteration + NumFramesBetweenIterations);
				}));
		}
	}

	// Wait on all async asset processing
	AddCommand(new FFunctionLatentCommand([] {
		return FAssetCompilingManager::Get().GetNumRemainingAssets() == 0;
		}));

	AddCommand(new FFunctionLatentCommand([LongAssetPath] {
		TRACE_BOOKMARK(TEXT("AssetCompilationComplete - %s"), *LongAssetPath);
		return true;
		}));

	return true;
}
