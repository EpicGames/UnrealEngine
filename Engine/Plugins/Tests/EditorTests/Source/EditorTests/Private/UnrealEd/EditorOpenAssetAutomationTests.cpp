// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/Paths.h"
#include "Misc/AutomationTest.h"
#include "Misc/App.h"
#include "UObject/Object.h"

#include "Tests/AutomationTestSettings.h"
#include "Tests/AutomationEditorCommon.h"
#include "Tests/AutomationCommon.h"

#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"

/**
 * Test to open the sub editor windows for a specified list of assets.
 * This list can be setup in the Editor Preferences window within the editor or the DefaultEngine.ini file for that particular project.
*/
IMPLEMENT_COMPLEX_AUTOMATION_TEST(FOpenAssetEditors, "Project.Editor.Open Assets", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter);

void FOpenAssetEditors::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	const UAutomationTestSettings* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
	for ( FSoftObjectPath AssetRef : AutomationTestSettings->AssetsToOpen )
	{
		OutBeautifiedNames.Add(AssetRef.GetAssetName());
		OutTestCommands.Add(AssetRef.GetLongPackageName());
	}
}

bool FOpenAssetEditors::RunTest(const FString& LongAssetPath)
{
	//start with all editors closed
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllAssetEditors();

	// below is all latent action, so before sending there, verify the asset exists
	UObject* Object = FSoftObjectPath(LongAssetPath).TryLoad();
	if ( Object == nullptr )
	{
		UE_LOGF(LogEditorAutomationTests, Error, "Failed to Open Asset '%ls'.", *LongAssetPath);
		return false;
	}

	AddCommand(new FOpenEditorForAssetCommand(LongAssetPath));
	AddCommand(new FWaitLatentCommand(0.5f));
	AddCommand(new FDelayedFunctionLatentCommand([WeakObjPtr=TWeakObjectPtr<UObject>(Object),LongAssetPath] {
		UObject* Obj = WeakObjPtr.Get();
		if (Obj == nullptr)
		{		
			// If the object was destroyed, attempt to find it if it was recreated
			if (!LongAssetPath.IsEmpty())
			{
				Obj = FindObject<UObject>(nullptr, *LongAssetPath);
			}

			if (Obj == nullptr)
			{
				UE_LOGF(LogEditorAutomationTests, Error, "Asset '%ls' did not exist when evaluating.", *LongAssetPath);
			}
		}
		if (Obj && Obj->GetOutermost()->IsDirty() )
		{
			UE_LOGF(LogEditorAutomationTests, Error, "Asset '%ls' was dirty after opening it.", *LongAssetPath);
		}
	}));
	AddCommand(new FCloseAllAssetEditorsCommand());
	AddCommand(new FDelayedFunctionLatentCommand([=] {
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}));

	return true;
}
