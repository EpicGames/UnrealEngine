// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/AutomationTest.h"
#include "Modules/ModuleManager.h"
#include "Misc/EngineVersion.h"
#include "Framework/Application/SlateApplication.h"

#include "Tests/AutomationTestSettings.h"
#include "Misc/PackageName.h"
#include "Tests/AutomationEditorCommon.h"
#include "Tests/AutomationCommon.h"
#include "Interfaces/IMainFrameModule.h"


//////////////////////////////////////////////////////////////////////////
//Struct used to hold the data for the Editor Performance test.
struct EditorPerfCaptureParameters
{
	//Basic Test Info
	FString MapName;
	int32 TestDuration;

	//Saved Performance Values
	float MapLoadTime;
	int64 Counter;
	TArray<float> AverageFPS;
	TArray<float> AverageFrameTime;
	TArray<float> UsedPhysical;
	TArray<float> AvailablePhysical;
	TArray<float> AvailableVirtual;
	TArray<float> UsedVirtual;
	TArray<float> PeakUsedVirtual;
	TArray<float> PeakUsedPhysical;
	TArray<FDateTime> TimeStamp;
	TArray<FString> FormattedTimeStamp;

	EditorPerfCaptureParameters()
		: MapName(TEXT("None"))
		, TestDuration(60)
		, MapLoadTime(0)
		, Counter(0)
	{
	}

};

//////////////////////////////////////////////////////////////////////////
//Editor Performance Functions

/**
* Dumps the information held within the EditorPerfCaptureParameters struct into a CSV file.
* @param EditorPerfStats is the name of the struct that holds the needed performance information.
*/
void EditorPerfDump(EditorPerfCaptureParameters& EditorPerfStats)
{
	UE_LOGF(LogEditorAutomationTests, Log, "Begin generating the editor performance charts.");

	//The file location where to save the data.
	FString DataFileLocation = FPaths::Combine(*FPaths::AutomationLogDir(), TEXT("Performance"), *EditorPerfStats.MapName);

	//Get the map load time (in seconds) from the text file that is created when the load map latent command is ran.
	EditorPerfStats.MapLoadTime = 0;
	FString MapLoadTimeFileLocation = FPaths::Combine(*DataFileLocation, TEXT("RAWMapLoadTime.txt"));
	if (FPaths::FileExists(MapLoadTimeFileLocation))
	{
		TArray<FString> SavedMapLoadTimes;
		FAutomationEditorCommonUtils::CreateArrayFromFile(MapLoadTimeFileLocation, SavedMapLoadTimes);
		EditorPerfStats.MapLoadTime = FCString::Atof(*SavedMapLoadTimes.Last());
	}
	
	//Filename for the RAW csv which holds the data gathered from a single test ran.
	FString RAWCSVFilePath = FString::Printf(TEXT("%s/RAW_%s_%s.csv"), *DataFileLocation, *EditorPerfStats.MapName, *FDateTime::Now().ToString());

	//Filename for the pretty csv file.
	FString PerfCSVFilePath = FString::Printf(TEXT("%s/%s_Performance.csv"), *DataFileLocation, *EditorPerfStats.MapName);

	//Create the raw csv and then add the title row it.
	FArchive* RAWCSVArchive = IFileManager::Get().CreateFileWriter(*RAWCSVFilePath);
	FString RAWCSVLine = (TEXT("Map Name, Changelist, Time Stamp, Map Load Time, Average FPS, Frame Time, Used Physical Memory, Used Virtual Memory, Used Peak Physical, Used Peak Virtual, Available Physical Memory, Available Virtual Memory\n"));
	RAWCSVArchive->Serialize(TCHAR_TO_ANSI(*RAWCSVLine), RAWCSVLine.Len());

	//Dump the stats from each run to the raw csv file and then close it.
	for (int32 i = 0; i < EditorPerfStats.TimeStamp.Num(); i++)
	{
		//If the raw file isn't available to write to then we'll fail back this test.
		if ( FAutomationEditorCommonUtils::IsArchiveWriteable(RAWCSVFilePath, RAWCSVArchive))
		{
			RAWCSVLine = FString::Printf(TEXT("%s,%s,%s,%.3f,%.1f,%.1f,%.0f,%.0f,%.0f,%.0f,%.0f,%.0f%s"), *EditorPerfStats.MapName, *FEngineVersion::Current().ToString(EVersionComponent::Changelist), *EditorPerfStats.FormattedTimeStamp[i], EditorPerfStats.MapLoadTime, EditorPerfStats.AverageFPS[i], EditorPerfStats.AverageFrameTime[i], EditorPerfStats.UsedPhysical[i], EditorPerfStats.UsedVirtual[i], EditorPerfStats.PeakUsedPhysical[i], EditorPerfStats.PeakUsedVirtual[i], EditorPerfStats.AvailablePhysical[i], EditorPerfStats.AvailableVirtual[i], LINE_TERMINATOR);
			RAWCSVArchive->Serialize(TCHAR_TO_ANSI(*RAWCSVLine), RAWCSVLine.Len());
		}
	}
	RAWCSVArchive->Close();

	//Get the final pretty data for the Performance csv file.
	float AverageFPS = FAutomationEditorCommonUtils::TotalFromFloatArray(EditorPerfStats.AverageFPS, true);
	float AverageFrameTime = FAutomationEditorCommonUtils::TotalFromFloatArray(EditorPerfStats.AverageFrameTime, true);
	float MemoryUsedPhysical = FAutomationEditorCommonUtils::TotalFromFloatArray(EditorPerfStats.UsedPhysical, true);
	float MemoryAvailPhysAvg = FAutomationEditorCommonUtils::TotalFromFloatArray(EditorPerfStats.AvailablePhysical, true);
	float MemoryAvailVirtualAvg = FAutomationEditorCommonUtils::TotalFromFloatArray(EditorPerfStats.AvailableVirtual, true);
	float MemoryUsedVirtualAvg = FAutomationEditorCommonUtils::TotalFromFloatArray(EditorPerfStats.UsedVirtual, true);
	float MemoryUsedPeak = FAutomationEditorCommonUtils::LargestValueInFloatArray(EditorPerfStats.PeakUsedPhysical);
	float MemoryUsedPeakVirtual = FAutomationEditorCommonUtils::LargestValueInFloatArray(EditorPerfStats.PeakUsedVirtual);

	//TestRunDuration is the length of time the test lasted in ticks.
	FTimespan TestRunDuration = (EditorPerfStats.TimeStamp.Last().GetTicks() - EditorPerfStats.TimeStamp[0].GetTicks()) + ETimespan::TicksPerSecond;

	//The performance csv file will be created if it didn't exist prior to the start of this test.
	if (!FPaths::FileExists(PerfCSVFilePath))
	{
		FArchive* FinalCSVArchive = IFileManager::Get().CreateFileWriter(*PerfCSVFilePath);
		if ( FAutomationEditorCommonUtils::IsArchiveWriteable(PerfCSVFilePath, FinalCSVArchive))
		{
			FString FinalCSVLine = (TEXT("Date, Map Name, Changelist, Test Run Time , Map Load Time, Average FPS, Average MS, Used Physical KB, Used Virtual KB, Used Peak Physcial KB, Used Peak Virtual KB, Available Physical KB, Available Virtual KB\n"));
			FinalCSVArchive->Serialize(TCHAR_TO_ANSI(*FinalCSVLine), FinalCSVLine.Len());
			FinalCSVArchive->Close();
		}
	}

	//Load the existing performance csv so that it doesn't get saved over and lost.
	FString OldPerformanceCSVFile;
	FFileHelper::LoadFileToString(OldPerformanceCSVFile, *PerfCSVFilePath);
	FArchive* FinalCSVArchive = IFileManager::Get().CreateFileWriter(*PerfCSVFilePath);
	if ( FAutomationEditorCommonUtils::IsArchiveWriteable(PerfCSVFilePath, FinalCSVArchive))
	{
		//Dump the old performance csv file data to the new csv file.
		FinalCSVArchive->Serialize(TCHAR_TO_ANSI(*OldPerformanceCSVFile), OldPerformanceCSVFile.Len());

		//Dump the pretty stats to the Performance CSV file and then close it so we can edit it while the engine is still running.
		FString FinalCSVLine = FString::Printf(TEXT("%s,%s,%s,%.0f,%.3f,%.1f,%.1f,%.0f,%.0f,%.0f,%.0f,%.0f,%.0f%s"), *FDateTime::Now().ToString(), *EditorPerfStats.MapName, *FEngineVersion::Current().ToString(EVersionComponent::Changelist), TestRunDuration.GetTotalSeconds(), EditorPerfStats.MapLoadTime, AverageFPS, AverageFrameTime, MemoryUsedPhysical, MemoryUsedVirtualAvg, MemoryUsedPeak, MemoryUsedPeakVirtual, MemoryAvailPhysAvg, MemoryAvailVirtualAvg, LINE_TERMINATOR);
		FinalCSVArchive->Serialize(TCHAR_TO_ANSI(*FinalCSVLine), FinalCSVLine.Len());
		FinalCSVArchive->Close();
	}

	//Display the test results to the user.
	UE_LOGF(LogEditorAutomationTests, Display, "AVG FPS: '%.1f'", AverageFPS);
	UE_LOGF(LogEditorAutomationTests, Display, "AVG Frame Time: '%.1f' ms", AverageFrameTime);
	UE_LOGF(LogEditorAutomationTests, Display, "AVG Used Physical Memory: '%.0f' kb", MemoryUsedPhysical);
	UE_LOGF(LogEditorAutomationTests, Display, "AVG Used Virtual Memory: '%.0f' kb", MemoryUsedVirtualAvg);
	UE_LOGF(LogEditorAutomationTests, Display, "Performance csv file is located here: %ls", *FPaths::ConvertRelativePathToFull(PerfCSVFilePath));
	UE_LOGF(LogEditorAutomationTests, Log, "Performance csv file is located here: %ls", *FPaths::ConvertRelativePathToFull(PerfCSVFilePath));
	UE_LOGF(LogEditorAutomationTests, Log, "Raw performance csv file is located here: %ls", *FPaths::ConvertRelativePathToFull(RAWCSVFilePath));
}

void CaptureEditorData(EditorPerfCaptureParameters& OutEditorPerfStats)
{
	//Capture the current time stamp and format it to YYYY-MM-DD HH:MM:SS.mmm.
	FDateTime CurrentDateAndTime = FDateTime::UtcNow();
	
	//Find the Average FPS
	//Clamp to avoid huge averages at startup or after hitches
	const float CurrentFPS = 1.0f / FSlateApplication::Get().GetAverageDeltaTime();
	const float ClampedFPS = (CurrentFPS < 0.0f || CurrentFPS > 4000.0f) ? 0.0f : CurrentFPS;
	OutEditorPerfStats.AverageFPS.Add(ClampedFPS);

	//Find the Frame Time in ms.
	//Clamp to avoid huge averages at startup or after hitches
	const float AverageMS = FSlateApplication::Get().GetAverageDeltaTime() * 1000.0f;
	const float ClampedMS = (AverageMS < 0.0f || AverageMS > 4000.0f) ? 0.0f : AverageMS;
	OutEditorPerfStats.AverageFrameTime.Add(ClampedMS);

	//Query OS for process memory used.
	FPlatformMemoryStats MemoryStats = FPlatformMemory::GetStats();
	OutEditorPerfStats.UsedPhysical.Add((float)MemoryStats.UsedPhysical / 1024);

	//Query OS for available physical memory
	OutEditorPerfStats.AvailablePhysical.Add((float)MemoryStats.AvailablePhysical / 1024);

	//Query OS for available virtual memory
	OutEditorPerfStats.AvailableVirtual.Add((float)MemoryStats.AvailableVirtual / 1024);

	//Query OS for used virtual memory
	OutEditorPerfStats.UsedVirtual.Add((float)MemoryStats.UsedVirtual / 1024);

	//Query OS for used Peak Used physical memory
	OutEditorPerfStats.PeakUsedPhysical.Add((float)MemoryStats.PeakUsedPhysical / 1024);

	//Query OS for used Peak Used virtual memory
	OutEditorPerfStats.PeakUsedVirtual.Add((float)MemoryStats.PeakUsedVirtual / 1024);

	//Capture the time stamp.
	FString FormatedTimeStamp = FString::Printf(TEXT("%04i-%02i-%02i %02i:%02i:%02i.%03i"), CurrentDateAndTime.GetYear(), CurrentDateAndTime.GetMonth(), CurrentDateAndTime.GetDay(), CurrentDateAndTime.GetHour(), CurrentDateAndTime.GetMinute(), CurrentDateAndTime.GetSecond(), CurrentDateAndTime.GetMillisecond());
	OutEditorPerfStats.FormattedTimeStamp.Add(FormatedTimeStamp);
	OutEditorPerfStats.TimeStamp.Add(CurrentDateAndTime);

}

//////////////////////////////////////////////////////////////////////////
//Editor Performance Latent Commands

/**
* This will capture the average FPS and Memory numbers over a duration of time.
* @param The name of the EditorPerfCaptureParameters struct that will be used to hold the data.
*/
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FEditorPerfCaptureCommand, EditorPerfCaptureParameters, EditorPerfStats);

/**
* This command grabs the FPS and Memory stats for the current editor session.
*/
bool FEditorPerfCaptureCommand::Update()
{
	float ElapsedTime = FPlatformTime::Seconds() - StartTime;
	
	if ((ElapsedTime <= (EditorPerfStats.TestDuration + 1.0f)))
	{
		if (((int64)ElapsedTime - EditorPerfStats.Counter) >= 1)
		{
			EditorPerfStats.Counter = ElapsedTime;
			CaptureEditorData(EditorPerfStats);
		}
		return false;
	}

	//Dump the performance data in a csv file.
	EditorPerfDump(EditorPerfStats);
	return true;
}

//////////////////////////////////////////////////////////////////////////
//Editor Performance Tests

/**
* Map Performance in Editor tests
* Grabs certain performance numbers and saves it to a file.
*/
IMPLEMENT_COMPLEX_AUTOMATION_TEST(FMapPerformanceInEditor, "Project.Performance.Map Performance in Editor", EAutomationTestFlags::EditorContext | EAutomationTestFlags::RequiresUser | EAutomationTestFlags::EngineFilter)

/**
* Requests a enumeration of all maps to be loaded
*/
void FMapPerformanceInEditor::GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const
{
	UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
	check(AutomationTestSettings);

	for ( const FEditorMapPerformanceTestDefinition& PerfDefinition : AutomationTestSettings->EditorPerformanceTestMaps )
	{
		const FSoftObjectPath& PerfMap = PerfDefinition.PerformanceTestmap;

		if ( PerfMap.IsValid() )
		{
			FString ShortName = FPackageName::GetShortName(PerfMap.GetLongPackageName());

			OutBeautifiedNames.Add(ShortName);
			OutTestCommands.Add(PerfMap.GetLongPackageName());
		}
	}
}

bool FMapPerformanceInEditor::RunTest(const FString& Parameters)
{
	//This is used to hold the data that is being captured.
	EditorPerfCaptureParameters EditorPerformanceData;

	//Get the map name from the parameters.
	FString MapName = Parameters;
	
	//Gets the main frame module to get the name of our current level.
	const IMainFrameModule& MainFrameModule = FModuleManager::GetModuleChecked< IMainFrameModule >("MainFrame");
	EditorPerformanceData.MapName = MainFrameModule.GetLoadedLevelName();

	//Duration variable will be used to indicate how long the test will run for.  Defaults to a minute.
	EditorPerformanceData.TestDuration = 60;

	//Get the base filename for the map that will be used.
	EditorPerformanceData.MapName = MapName;

	//Load Map and get the time it took to take to load the map.
	ADD_LATENT_AUTOMATION_COMMAND(FEditorLoadMap(MapName));

	//This gets the info we need from the automation test settings in the engine.ini.
	UAutomationTestSettings const* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
	check(AutomationTestSettings);

	//Now we find the test timer(aka duration) for our test.
	for (auto Iter = AutomationTestSettings->EditorPerformanceTestMaps.CreateConstIterator(); Iter; ++Iter)
	{
		if ( Iter->PerformanceTestmap.GetLongPackageName() == EditorPerformanceData.MapName )
		{
			EditorPerformanceData.TestDuration = ((Iter->TestTimer));
			//If the duration is equal to 0 then we simply warn the user that they need to set the test timer option for the performance test.
			//If the duration is less than 0 then we fail this test.
			if (EditorPerformanceData.TestDuration == 0)
			{
				UE_LOGF(LogEditorAutomationTests, Warning, "Please set the test timer for '%ls' in the automation preferences or engine.ini.", *EditorPerformanceData.MapName);
			}
			else
			if (EditorPerformanceData.TestDuration < 0)
			{
				UE_LOGF(LogEditorAutomationTests, Error, "Test timer preference option for '%ls' is less than 0.", *EditorPerformanceData.MapName);
				return false;
			}
		}
	}

	UE_LOGF(LogEditorAutomationTests, Log, "Running the performance capture test for %.0i seconds on %ls", EditorPerformanceData.TestDuration, *EditorPerformanceData.MapName);
	
	//Move the viewport views to the first bookmark
	ADD_LATENT_AUTOMATION_COMMAND(FChangeViewportToFirstAvailableBookmarkCommand);
	ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(0.5f));

	//Wait for shaders to finish compile.
	ADD_LATENT_AUTOMATION_COMMAND(FWaitForShadersToFinishCompiling);
	
	//Grab the performance numbers based on the duration.
	ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(0.5f));
	ADD_LATENT_AUTOMATION_COMMAND(FEditorPerfCaptureCommand(EditorPerformanceData));

	return true;
}
