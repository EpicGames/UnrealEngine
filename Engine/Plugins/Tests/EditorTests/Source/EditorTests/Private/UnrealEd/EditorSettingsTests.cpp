// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/AutomationTest.h"
#include "GenericPlatform/GenericApplication.h"
#include "Misc/EngineVersion.h"
#include "InputCoreTypes.h"
#include "Input/Events.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/InputBindingManager.h"
#include "Settings/EditorStyleSettings.h"
#include "Modules/ModuleManager.h"
#include "Editor.h"

// Automation
#include "Tests/AutomationEditorPromotionCommon.h"
#include "Tests/AutomationCommon.h"


#define LOCTEXT_NAMESPACE "EditorSettingsPromotionTests"

DEFINE_LOG_CATEGORY_STATIC(LogEditorSettingsTests, All, All);

//Latent commands
DEFINE_LATENT_AUTOMATION_COMMAND(FSettingsCheckForPIECommand);

//Tests
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditorSettingsKeybindingsTest, "System.Promotion.Editor.Settings.Keybindings", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEditorSettingsPreferencesTest, "System.Promotion.Editor.Settings.Preferences", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

/**
* Helper functions used by the settings automation tests
*/
namespace EditorSettingsTestUtils
{
	/**
	* Exports the current editor keybindings
	*
	* @param IniSettings - Settings to export
	* @param TargetFilename - The name of the file to export to
	*/
	static bool ExportSettings(FString& IniSettings, const FString& TargetFilename)
	{
		FInputBindingManager::Get().SaveInputBindings();
		GConfig->Flush(false, IniSettings);
		ECopyResult Expected = ECopyResult::COPY_OK;
		return ECopyResult::COPY_OK == IFileManager::Get().Copy(*TargetFilename, *IniSettings);
	}

	/**
	* Exports the current editor keybindings
	*
	* @param IniSettings - Settings to import to
	* @param TargetFilename - The name of the file to import
	*/
	static void ImportSettings(FString& IniSettings, const FString& SourceFilename)
	{
		GConfig->Flush(true, IniSettings);
		if (ECopyResult::COPY_OK == IFileManager::Get().Copy(*IniSettings, *SourceFilename))
		{
			GConfig->LoadFile(IniSettings);
		}
		else
		{
			UE_LOGF(LogEditorSettingsTests, Error, "Failed to load settings from %ls", *SourceFilename);
		}
	}

	/**
	* Exports the current editor keybindings
	*
	* @param TargetFilename - The name of the file to export to
	*/
	static void ExportKeybindings(const FString& TargetFilename)
	{
		FInputBindingManager::Get().SaveInputBindings();
		GConfig->Flush(false, GEditorKeyBindingsIni);
		IFileManager::Get().Copy(*TargetFilename, *GEditorKeyBindingsIni);
	}

	/**
	* Imports new editor keybindings  (Unused)
	*
	* @param TargetFilename - The name of the file to import from
	*/
	static void ImportKeybindings(const FString& TargetFilename)
	{
		GConfig->Flush(true, GEditorKeyBindingsIni);
		IFileManager::Get().Copy(*GEditorKeyBindingsIni, *TargetFilename);
		GConfig->LoadFile(GEditorKeyBindingsIni);
	}

	/**
	* Exports the current editor settings
	*
	* @param TargetFilename - The name of the file to export to
	*/
	static void ExportEditorSettings(const FString& TargetFilename)
	{
		FInputBindingManager::Get().SaveInputBindings();
		GConfig->Flush(false, GEditorPerProjectIni);
		IFileManager::Get().Copy(*TargetFilename, *GEditorPerProjectIni);
	}

	/**
	* Imports new editor settings (unused)
	*
	* @param TargetFilename - The name of the file to export to
	*/
	static void ImportEditorSettings(const FString& TargetFilename)
	{
		GConfig->Flush(true, GEditorPerProjectIni);
		IFileManager::Get().Copy(*GEditorPerProjectIni, *TargetFilename);
		GConfig->LoadFile(GEditorPerProjectIni);
	}

	/**
	* Creates a new keybinding chord and sets it for the supplied command and context
	*
	* @param CommandContext - The context of the command
	* @param Command - The command name to get
	* @param Key - The keybinding chord key
	* @param ModifierKey - The keybinding chord modifier key
	*/
	static FInputChord SetKeybinding(const FString& CommandContext, const FString& Command, const FKey Key, const EModifierKey::Type ModifierKey)
	{
		FInputChord NewChord(Key, ModifierKey);
		if (!FEditorPromotionTestUtilities::SetEditorKeybinding(CommandContext, Command, NewChord))
		{
			// Trigger a failure when used in an automated test
			UE_LOGF(LogEditorSettingsTests, Error, "Could not find keybinding for %ls using context %ls", *Command, *CommandContext);
		}
		return NewChord;
	}

	/**
	* Retrieves the current keybinding for a command and compares it against the expected binding.
	* Triggers an automation test failure if keybind cannot be retrieved or does not match expected binding.
	*
	* @param CommandContext - The context of the command
	* @param Command - The command name to get
	* @param ExpectedChord - The chord value to compare against
	*/
	static void CompareKeybindings(const FString& CommandContext, const FString& Command, FInputChord ExpectedChord)
	{
		FInputChord CurrentChord = FEditorPromotionTestUtilities::GetEditorKeybinding(CommandContext, Command);
		if (!CurrentChord.IsValidChord())
		{
			UE_LOGF(LogEditorSettingsTests, Error, "Could not find keybinding for %ls using context %ls", *Command, *CommandContext);
		}
		else
		{
			if (CurrentChord == ExpectedChord)
			{
				UE_LOGF(LogEditorSettingsTests, Display, "%ls keybinding correct.", *Command);
			}
			else
			{
				UE_LOGF(LogEditorSettingsTests, Error, "%ls keybinding incorrect.", *Command);
			}
		}
	}
}

/**
* Latent command to check if PIE is running
*/
bool FSettingsCheckForPIECommand::Update()
{
	if (GEditor->PlayWorld != NULL)
	{
		//Success
		UE_LOGF(LogEditorSettingsTests, Display, "PlayInEditor keyboard shortcut success");
		FEditorPromotionTestUtilities::EndPIE();
	}
	else
	{
		UE_LOGF(LogEditorSettingsTests, Error, "PlayInEditor keyboard shortcut failed");
	}
	return true;
}

/**
* Automation test that handles setting keybindings
*/
bool FEditorSettingsKeybindingsTest::RunTest(const FString& Parameters)
{
	UE_LOGF(LogEditorSettingsTests, Display, "Exporting Current keybindings and editor settings");

	// Ensure layers module is loaded
	FModuleManager::Get().LoadModule("Layers");

	//Export the original keybindings
	const FString TargetOriginalKeybindFile = FString::Printf(TEXT("%s/BuildPromotion/OriginalKeybindings-%d.ini"), *FPaths::AutomationDir(), FEngineVersion::Current().GetChangelist());
	EditorSettingsTestUtils::ExportKeybindings(TargetOriginalKeybindFile);

	//Cache original keybinding values, since reimporting settings doesn't reset values w/o a restart
	const FInputChord OriginalCreateChord = FEditorPromotionTestUtilities::GetEditorKeybinding(TEXT("LayersView"), TEXT("CreateEmptyLayer"));
	const FInputChord OriginalRenameChord = FEditorPromotionTestUtilities::GetEditorKeybinding(TEXT("LayersView"), TEXT("RequestRenameLayer"));
	const FInputChord OriginalPIEChord = FEditorPromotionTestUtilities::GetEditorKeybinding(TEXT("PlayWorld"), TEXT("RepeatLastPlay"));

	//New Editor Settings
	//Bind H to CreateEmptyLayer keybinding
	UE_LOGF(LogEditorSettingsTests, Display, "Binding create empty layer shortcut");
	FInputChord NewCreateChord = EditorSettingsTestUtils::SetKeybinding(TEXT("LayersView"), TEXT("CreateEmptyLayer"), EKeys::H, EModifierKey::None);
	
	//Bind J to RequestRenameLayer
	UE_LOGF(LogEditorSettingsTests, Display, "Binding request rename layer shortcut");
	FInputChord NewRenameChord = EditorSettingsTestUtils::SetKeybinding(TEXT("LayersView"), TEXT("RequestRenameLayer"), EKeys::J, EModifierKey::None);
	
	// Bind CTRL+L to PIE
	UE_LOGF(LogEditorSettingsTests, Display, "Binding play shortcut (PIE)");
	FInputChord NewPIEChord = EditorSettingsTestUtils::SetKeybinding(TEXT("PlayWorld"), TEXT("RepeatLastPlay"), EKeys::L, EModifierKey::Control);
	
	//Export the keybindings
	const FString TargetKeybindFile = FString::Printf(TEXT("%s/BuildPromotion/Keybindings-%d.ini"), *FPaths::AutomationDir(), FEngineVersion::Current().GetChangelist());
	UE_LOGF(LogEditorSettingsTests, Display, "Exporting keybind");
	EditorSettingsTestUtils::ExportKeybindings(TargetKeybindFile);

	// Verify keybindings were assigned correctly
	EditorSettingsTestUtils::CompareKeybindings(TEXT("LayersView"), TEXT("CreateEmptyLayer"), NewCreateChord);
	EditorSettingsTestUtils::CompareKeybindings(TEXT("LayersView"), TEXT("RequestRenameLayer"), NewRenameChord);
	EditorSettingsTestUtils::CompareKeybindings(TEXT("PlayWorld"), TEXT("RepeatLastPlay"), NewPIEChord);

	//Focus the main editor
	TArray< TSharedRef<SWindow> > AllWindows;
	FSlateApplication::Get().GetAllVisibleWindowsOrdered(AllWindows);
	if (AllWindows.Num() == 0)
	{
		// This can happen when the user has activated a different fullscreen application during the test. Mark the test as failed.
		AddError(TEXT("Can not focus the main editor"));
	}
	else
	{
		FSlateApplication::Get().ProcessWindowActivatedEvent(FWindowActivateEvent(FWindowActivateEvent::EA_Activate, AllWindows[0]));
		// TODO Automation Disabling this code until we understand why this no longer works.  Did it ever?
		#if (0)
		{
			//Send the PIE event
			FKeyEvent PIEKeyEvent(EKeys::L, FModifierKeysState(false, false, true, false, false, false, false, false, false), false, 0/*UserIndex*/, 0x4C, 0x4C);
			FSlateApplication::Get().ProcessKeyDownEvent(PIEKeyEvent);
			FSlateApplication::Get().ProcessKeyUpEvent(PIEKeyEvent);

			UE_LOGF(LogEditorSettingsTests, Display, "Sent PIE keyboard shortcut");

			ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(3.f));
			ADD_LATENT_AUTOMATION_COMMAND(FSettingsCheckForPIECommand());
		}
		#endif
	}

	//Import original keybindings and set changed binds back to cached values
	EditorSettingsTestUtils::ImportKeybindings(TargetOriginalKeybindFile);
	UE_LOGF(LogEditorSettingsTests, Display, "Reloaded original keybindings");

	FEditorPromotionTestUtilities::SetEditorKeybinding(TEXT("LayersView"), TEXT("CreateEmptyLayer"), OriginalCreateChord);
	FEditorPromotionTestUtilities::SetEditorKeybinding(TEXT("LayersView"), TEXT("RequestRenameLayer"), OriginalRenameChord);
	FEditorPromotionTestUtilities::SetEditorKeybinding(TEXT("PlayWorld"), TEXT("RepeatLastPlay"), OriginalPIEChord);

	return true;
}

/**
* Automation test that handles setting editor preferences
*/
bool FEditorSettingsPreferencesTest::RunTest(const FString& Parameters)
{
	UE_LOGF(LogEditorSettingsTests, Display, "Exporting Current keybindings and editor settings");

	//Export the original preferences
	const FString TargetOriginalPreferenceFile = FString::Printf(TEXT("%s/BuildPromotion/OriginalPreferences-%d.ini"), *FPaths::AutomationDir(), FEngineVersion::Current().GetChangelist());
	EditorSettingsTestUtils::ExportEditorSettings(TargetOriginalPreferenceFile);

	UEditorStyleSettings* EditorStyleSettings = GetMutableDefault<UEditorStyleSettings>();
	FString OldStyleSetting = FEditorPromotionTestUtilities::GetPropertyByName(EditorStyleSettings, TEXT("bUseSmallToolBarIcons"));
	
	FEditorPromotionTestUtilities::SetPropertyByName(EditorStyleSettings, TEXT("bUseSmallToolBarIcons"), TEXT("true"));
	UE_LOGF(LogEditorSettingsTests, Display, "Enabled UseSmallToolBarIcons");

	//Export the preferences
	const FString TargetPreferenceFile = FString::Printf(TEXT("%s/BuildPromotion/Preferences-%d.ini"), *FPaths::AutomationDir(), FEngineVersion::Current().GetChangelist());
	EditorSettingsTestUtils::ExportEditorSettings(TargetPreferenceFile);

	// TODO Improve this, to compare shots, and to only take images of the actual small toolbar icons.
	//Take a screenshot of the small icons
	//FEditorPromotionTestUtilities::TakeScreenshot(TEXT("Small Toolbar Icons"), FAutomationScreenshotOptions(EComparisonTolerance::Low));

	//Change the setting back
	FEditorPromotionTestUtilities::SetPropertyByName(EditorStyleSettings, TEXT("bUseSmallToolBarIcons"), OldStyleSetting);
	UE_LOGF(LogEditorSettingsTests, Display, "Set UseSmallToolBarIcons to original state");

	EditorSettingsTestUtils::ImportEditorSettings(TargetOriginalPreferenceFile);
	UE_LOGF(LogEditorSettingsTests, Display, "Reloaded original preferences");

	return true;
}

#undef LOCTEXT_NAMESPACE
