// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorTestsUtilityLibrary.h"
#include "CoreMinimal.h"
#include "Engine/Texture2D.h"
#include "MaterialOptions.h"
#include "IMeshMergeUtilities.h"
#include "MeshMergeModule.h"

#include "StaticMeshComponentAdapter.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/World.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Modules/ModuleManager.h"
#include "AutomationStaticMeshComponentAdapter.h"
#include "Algo/Transform.h"
#include "Materials/Material.h"
#include "TextureCompiler.h"
#include "WidgetBlueprint.h"
#include "Blueprint/WidgetTree.h"
#include "Components/Widget.h"

DECLARE_LOG_CATEGORY_CLASS(LogEditorTestsUtilityLibrary, Log, All)

namespace
{
	void WaitForTextures(const UMaterialInterface* Material)
	{
		TArray<UTexture*> MaterialTextures;
		if (Material != nullptr)
		{
			Material->GetUsedTextures(MaterialTextures);

			FTextureCompilingManager::Get().FinishCompilation(MaterialTextures);

			// Force load materials used by the current material
			for (UTexture* Texture : MaterialTextures)
			{
				if (Texture != NULL)
				{
					UTexture2D* Texture2D = Cast<UTexture2D>(Texture);
					if (Texture2D)
					{
						Texture2D->SetForceMipLevelsToBeResident(30.0f);
						Texture2D->WaitForStreaming();
					}
				}
			}
		}
	}
}

void UEditorTestsUtilityLibrary::BakeMaterialsForComponent(UStaticMeshComponent* InStaticMeshComponent, const UMaterialOptions* MaterialOptions, const UMaterialMergeOptions* MaterialMergeOptions)
{
	if (InStaticMeshComponent != nullptr && InStaticMeshComponent->GetStaticMesh() != nullptr)
	{
		FModuleManager::Get().LoadModule("MaterialBaking");
		// Retrieve settings object
		UAssetBakeOptions* AssetOptions = GetMutableDefault<UAssetBakeOptions>();
		TArray<TWeakObjectPtr<UObject>> Objects = {
			MakeWeakObjectPtr(const_cast<UMaterialMergeOptions*>(MaterialMergeOptions)),
			MakeWeakObjectPtr(AssetOptions),
			MakeWeakObjectPtr(const_cast<UMaterialOptions*>(MaterialOptions))
		};

		FAutomationStaticMeshComponentAdapter Adapter(InStaticMeshComponent);
		const IMeshMergeUtilities& MeshMergeUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();
		MeshMergeUtilities.BakeMaterialsForComponent(Objects, &Adapter);

		InStaticMeshComponent->MarkRenderStateDirty();
		InStaticMeshComponent->MarkRenderTransformDirty();
		InStaticMeshComponent->MarkRenderDynamicDataDirty();

		const int32 NumMaterials = InStaticMeshComponent->GetNumMaterials();
		for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; ++MaterialIndex)
		{
			UMaterialInterface* Material = InStaticMeshComponent->GetMaterial(MaterialIndex);
			WaitForTextures(Material);
		}
	}
}

void UEditorTestsUtilityLibrary::MergeStaticMeshComponents(const TArray<UStaticMeshComponent*>& InStaticMeshComponents, const FMeshMergingSettings& MergeSettings, const bool bReplaceActors, TArray<int32>& OutLODIndices)
{
	FModuleManager::Get().LoadModule("MaterialBaking");

	const IMeshMergeUtilities& MeshMergeUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();

	// Convert array of StaticMeshComponents to PrimitiveComponents
	TArray<UPrimitiveComponent*> PrimCompsToMerge;
	Algo::TransformIf(InStaticMeshComponents, PrimCompsToMerge, 
		[](UStaticMeshComponent* StaticMeshComp) { return StaticMeshComp != nullptr && StaticMeshComp->GetStaticMesh() != nullptr; },
		[](UStaticMeshComponent* StaticMeshComp) { return StaticMeshComp; });

	if (!PrimCompsToMerge.IsEmpty())
	{
		UWorld* World = PrimCompsToMerge[0]->GetWorld();

		TArray<UObject*> Output;
		FVector OutPosition;
		MeshMergeUtilities.MergeComponentsToStaticMesh(PrimCompsToMerge, World, MergeSettings, nullptr, GetTransientPackage(), InStaticMeshComponents[0]->GetStaticMesh()->GetName(), Output, OutPosition, 1.0f, false);

		UObject** MaterialPtr = Output.FindByPredicate([](UObject* Object) { return Object->IsA<UMaterial>(); });
		if (MaterialPtr)
		{
			if (UMaterial* MergedMaterial = Cast<UMaterial>(*MaterialPtr))
			{
				WaitForTextures(MergedMaterial);
			}
		}

		// Place new mesh in the world
		if (bReplaceActors)
		{
			UObject** ObjectPtr = Output.FindByPredicate([](const UObject* Object) {return Object->IsA<UStaticMesh>(); });
			if (ObjectPtr != nullptr && *ObjectPtr != nullptr)
			{
				if (UStaticMesh* MergedMesh = CastChecked<UStaticMesh>(*ObjectPtr))
				{
					for (int32 Index = 0; Index < MergedMesh->GetNumLODs(); ++Index)
					{
						OutLODIndices.Add(Index);
					}

					FActorSpawnParameters Params;
					Params.OverrideLevel = World->PersistentLevel;
					FRotator MergedActorRotation(ForceInit);

					AStaticMeshActor* MergedActor = World->SpawnActor<AStaticMeshActor>(OutPosition, MergedActorRotation, Params);
					MergedActor->SetMobility(EComponentMobility::Movable);
					MergedActor->SetActorLabel(Output[0]->GetName());
					MergedActor->GetStaticMeshComponent()->SetStaticMesh(MergedMesh);

					TArray<AActor*> OwningActors;
					for (UStaticMeshComponent* Component : InStaticMeshComponents)
					{
						OwningActors.AddUnique(Component->GetOwner());
					}

					// Remove source actors
					for (AActor* Actor : OwningActors)
					{
						Actor->Destroy();
					}
				}
			}
		}
	}
}

void UEditorTestsUtilityLibrary::CreateProxyMesh(const TArray<UStaticMeshComponent*>& InStaticMeshComponents, const struct FMeshProxySettings& ProxySettings)
{
	TArray<UStaticMeshComponent*> StaticMeshComponents;
	Algo::CopyIf(InStaticMeshComponents, StaticMeshComponents, [](UStaticMeshComponent* StaticMeshComp) { return StaticMeshComp != nullptr && StaticMeshComp->GetStaticMesh() != nullptr; });

	if (StaticMeshComponents.IsEmpty())
	{
		return;
	}

	UWorld* World = StaticMeshComponents[0]->GetWorld();

	// Generate proxy mesh and proxy material assets 
	FCreateProxyDelegate ProxyDelegate;
	ProxyDelegate.BindLambda([&](const FGuid Guid, TArray<UObject*>& AssetsToSync)
	{
		UStaticMesh* ProxyMesh = nullptr;
		if (!AssetsToSync.FindItemByClass(&ProxyMesh))
		{
			UE_LOGF(LogEditorTestsUtilityLibrary, Error, "CreateProxyMesh failed. No mesh was created.");
			return;
		}

		UMaterialInterface* ProxyMaterial = nullptr;
		if (!AssetsToSync.FindItemByClass(&ProxyMaterial))
		{
			WaitForTextures(ProxyMaterial);
		}

		// Place new mesh in the world (on a new actor)
		FActorSpawnParameters Params;
		Params.OverrideLevel = World->PersistentLevel;
		AStaticMeshActor* MergedActor = World->SpawnActor<AStaticMeshActor>(Params);
		if (!MergedActor)
		{
			UE_LOGF(LogEditorTestsUtilityLibrary, Error, "CreateProxyMesh failed. Internal error while creating the merged actor.");
			return;
		}

		MergedActor->SetMobility(EComponentMobility::Movable);
		MergedActor->SetActorLabel(TEXT("Proxy_Actor"));
		MergedActor->GetStaticMeshComponent()->SetStaticMesh(ProxyMesh);		

		TArray<AActor*> OwningActors;
		for (UStaticMeshComponent* Component : StaticMeshComponents)
		{
			OwningActors.AddUnique(Component->GetOwner());
		}

		// Remove source actors
		for (AActor* Actor : OwningActors)
		{
			Actor->Destroy();
		}
	});

	const IMeshMergeUtilities& MeshMergeUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();
	MeshMergeUtilities.CreateProxyMesh(StaticMeshComponents, ProxySettings, GetTransientPackage(), TEXT("ProxyMesh"), FGuid::NewGuid(), ProxyDelegate);
}

UWidget* UEditorTestsUtilityLibrary::GetChildEditorWidgetByName(UWidgetBlueprint* WidgetBlueprint, FString Name)
{
	if (!ensure(WidgetBlueprint))
	{
		return nullptr;
	}

	UObject* Child = FindObject<UObject>(WidgetBlueprint->WidgetTree, *Name);
	return Cast<UWidget>(Child);
}

void UEditorTestsUtilityLibrary::SetEditorWidgetNavigationRule(UWidget* Widget, EUINavigation Nav, EUINavigationRule Rule)
{
	if (!Widget)
	{
		return;
	}

	// mimicking the FWidgetNavigationCustomization, this subobject exercises a specific edge case 
	// within the reinstancing code, and its coverage of reinstancing that I'm interested in- not 
	// so much editing of the subobject:
	UWidgetNavigation* WidgetNavigation = Widget->Navigation;
	if (!WidgetNavigation)
	{
		Widget->Navigation = NewObject<UWidgetNavigation>(Widget);
		WidgetNavigation = Widget->Navigation;
		WidgetNavigation->SetFlags(RF_Transactional);
		if (Widget->IsTemplate())
		{
			EObjectFlags TemplateFlags = Widget->GetMaskedFlags(RF_PropagateToSubObjects) | RF_DefaultSubObject;
			WidgetNavigation->SetFlags(TemplateFlags);
		}
	}

	FWidgetNavigationData* DirectionNavigation = nullptr;

	switch (Nav)
	{
	case EUINavigation::Left:
		DirectionNavigation = &WidgetNavigation->Left;
		break;
	case EUINavigation::Right:
		DirectionNavigation = &WidgetNavigation->Right;
		break;
	case EUINavigation::Up:
		DirectionNavigation = &WidgetNavigation->Up;
		break;
	case EUINavigation::Down:
		DirectionNavigation = &WidgetNavigation->Down;
		break;
	case EUINavigation::Next:
		DirectionNavigation = &WidgetNavigation->Next;
		break;
	case EUINavigation::Previous:
		DirectionNavigation = &WidgetNavigation->Previous;
		break;
	default:
		// Should not be possible.
		check(false);
		return;
	}

	DirectionNavigation->Rule = Rule;
}

EUINavigationRule UEditorTestsUtilityLibrary::GetEditorWidgetNavigationRule(UWidget* Widget, EUINavigation Nav)
{
	if (!Widget || !Widget->Navigation)
	{
		return EUINavigationRule::Escape;
	}

	UWidgetNavigation* WidgetNavigation = Widget->Navigation;

	switch (Nav)
	{
	case EUINavigation::Left:
		return WidgetNavigation->Left.Rule;
	case EUINavigation::Right:
		return WidgetNavigation->Right.Rule;
	case EUINavigation::Up:
		return WidgetNavigation->Up.Rule;
	case EUINavigation::Down:
		return WidgetNavigation->Down.Rule;
	case EUINavigation::Next:
		return WidgetNavigation->Next.Rule;
	case EUINavigation::Previous:
		return WidgetNavigation->Previous.Rule;
	default:
		// Should not be possible.
		check(false);
	}
	return EUINavigationRule::Escape;
}

