// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "ShowFlags.h"
#include "Editor.h"
#include "LevelEditorViewport.h"

namespace FViewportTestHelper
{
	/**
	* Finds all of the available show flags and sets the test command and names.
	*/
	void GetViewportFlagTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands)
	{
		uint32 index = 0;
		FString AvailableFlag;
		FString GroupName;
		EShowFlagGroup FlagGroup = SFG_Normal;

		while (true)
		{
			AvailableFlag = FEngineShowFlags::FindNameByIndex(index);
			if (AvailableFlag.IsEmpty()) return;

			FlagGroup = FEngineShowFlags::FindShowFlagGroup(*AvailableFlag);

			switch (FlagGroup)
			{
			case SFG_Normal:
				GroupName = TEXT("Normal");
				break;
			case SFG_Advanced:
				GroupName = TEXT("Advanced");
				break;
			case SFG_PostProcess:
				GroupName = TEXT("Post Process");
				break;
			case SFG_CollisionModes:
				GroupName = TEXT("Collision Modes");
				break;
			case SFG_Developer:
				GroupName = TEXT("Developer");
				break;
			case SFG_Visualize:
				GroupName = TEXT("Visualize");
				break;
			case SFG_LightTypes:
				GroupName = TEXT("Light Types");
				break;
			case SFG_LightingComponents:
				GroupName = TEXT("Light Components");
				break;
			case SFG_LightingFeatures:
				GroupName = TEXT("Light Features");
				break;
			case SFG_Lumen:
				GroupName = TEXT("Lumen");
				break;
			case SFG_MegaLights:
				GroupName = TEXT("MegaLights");
				break;
			case SFG_Nanite:
				GroupName = TEXT("Nanite");
				break;
			case SFG_Hidden:
			case SFG_Transient:
				GroupName = TEXT("Hidden");
				break;
			case SFG_Max:
				break;
			case SFG_Custom:
				GroupName = TEXT("Custom");
			default:
				GroupName = TEXT("Post Process");
				break;
			}

			int32 FlagIndex = FEngineShowFlags::FindIndexByName(*AvailableFlag);

			AvailableFlag = FString::Printf(TEXT("%s.%s"), *GroupName, *AvailableFlag);

			OutBeautifiedNames.Add(AvailableFlag);
			OutTestCommands.Add(FString::FromInt(FlagIndex));

			GroupName.Empty();
			AvailableFlag.Empty();
			FlagGroup = SFG_Normal;
			index++;
		}
	}

	/**
	* Returns the current flags state for the given show flag index.
	*/
	bool GetPerspectiveOriginalFlagstate(const int32& InFlagIndex, int32& OutViewportClientNumber)
	{
		// Switch the view port to perspective.
		FLevelEditorViewportClient* ViewportClient;
		for (int32 i = 0; i < GEditor->GetLevelViewportClients().Num(); i++)
		{
			ViewportClient = GEditor->GetLevelViewportClients()[i];
			if (ViewportClient->IsOrtho()) continue;
			
			OutViewportClientNumber = i;
			
			// Get the original show flag state for this show flag index.
			return ViewportClient->EngineShowFlags.GetSingleFlag(InFlagIndex);
		}

		return false;
	}
}

/**
* Unit test for enabling the view ports show flags
*/
IMPLEMENT_COMPLEX_AUTOMATION_TEST( FViewportShowflagsToggleOnTest, "Editor.Viewport.Showflags.Toggle On", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter );

void FViewportShowflagsToggleOnTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	FViewportTestHelper::GetViewportFlagTests(OutBeautifiedNames, OutTestCommands);
}

bool FViewportShowflagsToggleOnTest::RunTest( const FString& Parameters )
{
	// SETUP //	
	// Get the index for the flag and convert it back to an int.
	int32 FlagIndex = FCString::Atoi( *Parameters );
	int32 ViewportClientNumber = 0;
	bool OrigianlShowFlagState = FViewportTestHelper::GetPerspectiveOriginalFlagstate( FlagIndex, ViewportClientNumber );
	FLevelEditorViewportClient* ViewportClient = GEditor->GetLevelViewportClients()[ViewportClientNumber];

	// TEST //
	// Set the show flag to be enabled.
	ViewportClient->EngineShowFlags.SetSingleFlag(FlagIndex, true);

	// VERIFY //
	bool NewFlagState = ViewportClient->EngineShowFlags.GetSingleFlag(FlagIndex);
	TestTrue(TEXT("The showflag state was not set to true."), NewFlagState);

	// TEARDOWN //
	// This sets the view port back to it's default show flag value.
	ViewportClient->EngineShowFlags.SetSingleFlag(FlagIndex, OrigianlShowFlagState);

	return true;
}


/**
* Unit test for disabling the view ports show flags
*/
IMPLEMENT_COMPLEX_AUTOMATION_TEST(FViewportShowflagsToggleOffTest, "Editor.Viewport.Showflags.Toggle Off", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

void FViewportShowflagsToggleOffTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	FViewportTestHelper::GetViewportFlagTests(OutBeautifiedNames, OutTestCommands);
}

bool FViewportShowflagsToggleOffTest::RunTest(const FString& Parameters)
{
	// SETUP //	
	// Get the index for the flag and convert it back to an int.
	int32 FlagIndex = FCString::Atoi(*Parameters);
	int32 ViewportClientNumber = 0;
	bool OrigianlShowFlagState = FViewportTestHelper::GetPerspectiveOriginalFlagstate(FlagIndex, ViewportClientNumber);
	FLevelEditorViewportClient* ViewportClient = GEditor->GetLevelViewportClients()[ViewportClientNumber];

	// TEST //
	// Set the show flag to be disabled.
	ViewportClient->EngineShowFlags.SetSingleFlag(FlagIndex, false);

	// VERIFY //
	bool NewFlagState = ViewportClient->EngineShowFlags.GetSingleFlag(FlagIndex);
	TestFalse(TEXT("The showflag state was not set to true."), NewFlagState);

	// TEARDOWN //
	// This sets the view port back to it's default show flag value.
	ViewportClient->EngineShowFlags.SetSingleFlag(FlagIndex, OrigianlShowFlagState);

	return true;
}
