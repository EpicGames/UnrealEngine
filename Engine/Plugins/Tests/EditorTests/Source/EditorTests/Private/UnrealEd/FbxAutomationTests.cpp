// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimSequence.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/Skeleton.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "CoreMinimal.h"
#include "Editor/EditorEngine.h"
#include "Editor/Transactor.h"
#include "EditorReimportHandler.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StaticMesh.h"
#include "Factories/FbxAnimSequenceImportData.h"
#include "Factories/FbxFactory.h"
#include "Factories/FbxImportUI.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "Factories/FbxStaticMeshImportData.h"
#include "Factories/FbxTextureImportData.h"
#include "Factories/ReimportFbxSkeletalMeshFactory.h"
#include "Factories/ReimportFbxStaticMeshFactory.h"
#include "FbxMeshUtils.h"
#include "HAL/FileManager.h"
#include "IAssetTools.h"
#include "InterchangeHelper.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/AutomationTest.h"
#include "ObjectTools.h"
#include "Modules/ModuleManager.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "SkinWeightsUtilities.h"
#include "StaticMeshResources.h"
#include "Templates/Function.h"
#include "Tests/FbxAutomationCommon.h"
#include "UObject/SavePackage.h"

extern UNREALED_API UEditorEngine* GEditor;

//////////////////////////////////////////////////////////////////////////

/**
* FFbxImportAssetsAutomationTest
* Test that attempts to import .fbx files and verify that the result match the expectation (import options and result expectation are in a .json file) within the unit test directory in a sub-folder
* specified in the engine.ini file "AutomationTesting->FbxImportTestPath". Cannot be run in a commandlet
* as it executes code that routes through Slate UI.
*/
IMPLEMENT_COMPLEX_AUTOMATION_TEST(FFbxImportAssetsAutomationTest, "Editor.Import.Fbx", (EAutomationTestFlags::EditorContext | EAutomationTestFlags::NonNullRHI | EAutomationTestFlags::EngineFilter))

class FFbxImportAssetsAutomationTestHelper
{
public:
	static UAnimSequence* GetImportedAnimSequence(TArray<FAssetData>& ImportedAssets, FAutomationTestExecutionInfo& ExecutionInfo, const FString& FormatedErrorMessagePrefix)
	{
		UAnimSequence* AnimSequence = nullptr;
		for (const FAssetData& AssetData : ImportedAssets)
		{
			UObject* ImportedAsset = AssetData.GetAsset();
			if (ImportedAsset->IsA(UAnimSequence::StaticClass()))
			{
				AnimSequence = Cast<UAnimSequence>(ImportedAsset);
			}
		}
		if (AnimSequence == nullptr)
		{
			ExecutionInfo.AddError(FString::Printf(TEXT("%s no animation was imported"), *FormatedErrorMessagePrefix));
		}

		return AnimSequence;
	}

	static bool GetImportedCustomCurveKey(TArray<FAssetData>& ImportedAssets, FAutomationTestExecutionInfo& ExecutionInfo, const FString& FormatedErrorMessagePrefix, const FFbxTestPlanExpectedResult& ExpectedResult, FRichCurveKey& OutCurveKey)
	{
		if (UAnimSequence* AnimSequence = FFbxImportAssetsAutomationTestHelper::GetImportedAnimSequence(ImportedAssets, ExecutionInfo, FormatedErrorMessagePrefix))
		{
			if (ExpectedResult.ExpectedPresetsDataString.Num() < 1)
			{
				ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 1 string data (Expected Custom Curve Name)"),
					*FormatedErrorMessagePrefix));
				return false;
			}

			if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 1)
			{
				ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 1 integer data (Expected Custom Curve Key index)"),
					*FormatedErrorMessagePrefix));
				return false;
			}

			const FString& CurveName = ExpectedResult.ExpectedPresetsDataString[0];
			const FFloatCurve* FloatCurve = static_cast<const FFloatCurve*>(AnimSequence->GetCurveData().GetCurveData(FName(*CurveName), ERawCurveTrackTypes::RCT_Float));
			if (FloatCurve == nullptr)
			{
				ExecutionInfo.AddError(FString::Printf(TEXT("%s no custom curve named %s was imported"),
					*FormatedErrorMessagePrefix, *CurveName));
				return false;
			}

			const FRichCurve& RichCurve = FloatCurve->FloatCurve;
			const TArray<FRichCurveKey>& Keys = RichCurve.GetConstRefOfKeys();
			int KeyIndex = ExpectedResult.ExpectedPresetsDataInteger[0];
			if (!Keys.IsValidIndex(KeyIndex))
			{
				ExecutionInfo.AddError(FString::Printf(TEXT("%s no key at the index %i was imported"),
					*FormatedErrorMessagePrefix, KeyIndex));
				return false;
			}

			OutCurveKey = Keys[KeyIndex];
			return true;
		}

		return false;
	}
};

/**
* Requests a enumeration of all sample assets to import
*/
void FFbxImportAssetsAutomationTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const
{
	FString ImportTestDirectory;
	check(GConfig);
	GConfig->GetString(TEXT("AutomationTesting.FbxImport"), TEXT("FbxImportTestPath"), ImportTestDirectory, GEngineIni);
	ImportTestDirectory = FPaths::Combine(FPaths::ConvertRelativePathToFull(FPaths::EngineDir()), ImportTestDirectory);

	// Find all files in the GenericImport directory
	TArray<FString> FilesInDirectory;
	IFileManager::Get().FindFilesRecursive(FilesInDirectory, *ImportTestDirectory, TEXT("*.*"), true, false);

	// Scan all the found files, use only .fbx file
	for (TArray<FString>::TConstIterator FileIter(FilesInDirectory); FileIter; ++FileIter)
	{
		FString Filename(*FileIter);
		FString Ext = FPaths::GetExtension(Filename, true);
		if (Ext.Compare(TEXT(".fbx"), ESearchCase::IgnoreCase) == 0)
		{
			FString FileString = *FileIter;
			FString FileTestName = FPaths::GetBaseFilename(Filename);
			if (FileTestName.Len() > 6)
			{
				FString FileEndSuffixe = FileTestName.RightChop(FileTestName.Find(TEXT("_lod"), ESearchCase::IgnoreCase, ESearchDir::FromEnd));
				FString LodNumber = FileEndSuffixe.RightChop(4);
				FString LodBaseSuffixe = FileEndSuffixe.LeftChop(2);
				if (LodBaseSuffixe.Compare(TEXT("_lod"), ESearchCase::IgnoreCase) == 0)
				{
					if (LodNumber.Compare(TEXT("00")) != 0)
					{
						//Don't add lodmodel has test
						continue;
					}
				}
			}
			if (FileTestName.Len() > 4 && FileTestName.EndsWith(TEXT("_alt"), ESearchCase::IgnoreCase))
			{
				//Dont add alternate skinning has test
				continue;
			}
			OutBeautifiedNames.Add(FileTestName);
			OutTestCommands.Add(FileString);
		}
	}
}

FString GetFormatedMessageErrorInTestData(FString FileName, FString TestPlanName, FString ExpectedResultName, int32 ExpectedResultIndex)
{
	return FString::Printf(TEXT("%s->%s: Error in the test data, %s[%d]"),
		*FileName, *TestPlanName, *ExpectedResultName, ExpectedResultIndex);
}

FString GetFormatedMessageErrorInExpectedResult(FString FileName, FString TestPlanName, FString ExpectedResultName, int32 ExpectedResultIndex)
{
	return FString::Printf(TEXT("%s->%s: Wrong Expected Result, %s[%d] dont match expected data"),
		*FileName, *TestPlanName, *ExpectedResultName, ExpectedResultIndex);
}

// Codegen optimization degenerates for very long functions like RunTest when combined with the invokation of lots of FORCEINLINE methods.
// We don't need this code to be particularly fast anyway. The other way to improve this code would be to split the test in multiple functions.
BEGIN_FUNCTION_BUILD_OPTIMIZATION

/**
* Execute the generic import test
*
* @param Parameters - Should specify the asset to import
* @return	TRUE if the test was successful, FALSE otherwise
*/
bool FFbxImportAssetsAutomationTest::RunTest(const FString& Parameters)
{
	TArray<FString> CurFileToImport;
	CurFileToImport.Add(*Parameters);
	FString CleanFilename = FPaths::GetCleanFilename(CurFileToImport[0]);
	FString BaseFilename = FPaths::GetBaseFilename(CurFileToImport[0]);
	FString Ext = FPaths::GetExtension(CurFileToImport[0], true);
	FString FileOptionAndResult = CurFileToImport[0];
	if(!FileOptionAndResult.RemoveFromEnd(*Ext))
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("%s: Cannot find the information file (.json)"), *CleanFilename)));
		return false;
	}
	FileOptionAndResult += TEXT(".json");
	
	TArray<UFbxTestPlan*> TestPlanArray;
	
	if (!IFileManager::Get().FileExists(*FileOptionAndResult))
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("%s: Cannot find the information file (.json)."), *CleanFilename)));
		return false;
	}

	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

	static const auto CVarInterchangeFbx = IConsoleManager::Get().FindConsoleVariable(TEXT("Interchange.FeatureFlags.Import.FBX"));
	bool IsInterchangeFbxEnabled = CVarInterchangeFbx->GetBool();
	UE::Interchange::FScopedLambda IsInterchangeEnabledGuard([&IsInterchangeFbxEnabled]()
		{ 
			CVarInterchangeFbx->Set(IsInterchangeFbxEnabled, ECVF_SetByConsole);
		});
	//Make sure interchange is disabled for fbx
	CVarInterchangeFbx->Set(false, ECVF_SetByConsole);

	FString PackagePath;
	check(GConfig);
	GConfig->GetString(TEXT("AutomationTesting.FbxImport"), TEXT("FbxImportTestPackagePath"), PackagePath, GEngineIni);
	
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	
	{
		TArray<FAssetData> AssetsToDelete;
		AssetRegistryModule.Get().GetAssetsByPath(FName(*PackagePath), AssetsToDelete, true);
		ObjectTools::DeleteAssets(AssetsToDelete, false);
	}

	//Add a folder with the file name
	FString ImportAssetPath = PackagePath + TEXT("/") + BaseFilename;
	//Read the fbx options from the .json file and fill the ImportUI
	FbxAutomationTestsAPI::ReadFbxOptions(FileOptionAndResult, TestPlanArray);

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	bool CurTestSuccessful = (TestPlanArray.Num() > 0);
	TArray<UObject*> GlobalImportedObjects;
	for (UFbxTestPlan* TestPlan : TestPlanArray)
	{
		checkSlow(TestPlan->ImportUI);

		int32 WarningNum = ExecutionInfo.GetWarningTotal();
		int32 ErrorNum = ExecutionInfo.GetErrorTotal();
		TArray<UObject*> ImportedObjects;
		switch (TestPlan->Action)
		{
			case EFBXTestPlanActionType::Import:
			case EFBXTestPlanActionType::ImportReload:
			{
				//Create a factory and set the options
				UFbxFactory* FbxFactory = NewObject<UFbxFactory>(UFbxFactory::StaticClass());
				FbxFactory->AddToRoot();
				
				FbxFactory->ImportUI = TestPlan->ImportUI;
				//Skip the auto detect type on import, the test set a specific value
				FbxFactory->SetDetectImportTypeOnImport(false);
				
				if (FbxFactory->ImportUI->bImportAsSkeletal)
				{
					FbxFactory->ImportUI->MeshTypeToImport = FBXIT_SkeletalMesh;
					//AUtomation test do not support yet skeletalmesh geometry
					FbxFactory->ImportUI->SkeletalMeshImportData->ImportContentType = EFBXImportContentType::FBXICT_All;
				}

				//Import the test object
				ImportedObjects = AssetToolsModule.Get().ImportAssets(CurFileToImport, ImportAssetPath, FbxFactory);
				if (TestPlan->Action == EFBXTestPlanActionType::ImportReload)
				{
					TArray<FString> FullAssetPaths;
					
					TArray<FAssetData> ImportedAssets;
					AssetRegistryModule.Get().GetAssetsByPath(FName(*ImportAssetPath), ImportedAssets, true);
					for (const FAssetData& AssetData : ImportedAssets)
					{
						UObject *Asset = AssetData.GetAsset();
						if (Asset != nullptr)
						{
							if (ImportedObjects.Contains(Asset))
							{
								FullAssetPaths.Add(Asset->GetPathName());
							}
							FString PackageName = Asset->GetOutermost()->GetPathName();
							Asset->MarkPackageDirty();
							FSavePackageArgs SaveArgs;
							SaveArgs.TopLevelFlags = RF_Standalone;
							SaveArgs.SaveFlags = SAVE_NoError;
							UPackage::SavePackage(Asset->GetOutermost(), Asset,
								*FPackageName::LongPackageNameToFilename(PackageName, FPackageName::GetAssetPackageExtension()),
								SaveArgs);
						}
					}
					for (const FAssetData& AssetData : ImportedAssets)
					{
						UPackage *Package = AssetData.GetPackage();
						if (Package != nullptr)
						{
							for (FThreadSafeObjectIterator It; It; ++It)
							{
								UObject* ExistingObject = *It;
								if ((ExistingObject->GetOutermost() == Package) )
								{
									ExistingObject->ClearFlags(RF_Standalone | RF_Public);
									ExistingObject->RemoveFromRoot();
									ExistingObject->MarkAsGarbage();
								}
						
							}
						}
					}

					if ((GEditor != nullptr) && (GEditor->Trans != nullptr))
					{
						GEditor->Trans->Reset(FText::FromString("Discard undo history during FBX Automation testing."));
					}


					ImportedObjects.Empty();

					CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

					for (const FAssetData& AssetData : ImportedAssets)
					{
						UPackage *Package = AssetData.GetPackage();
						if (Package != nullptr)
						{
							if (!Package->IsFullyLoaded())
							{
								Package->FullyLoad();
							}
						}
					}
					
					//Set back the importObjects
					for (FString PathName : FullAssetPaths)
					{
						UStaticMesh* FoundMesh = LoadObject<UStaticMesh>(NULL, *PathName, NULL, LOAD_Quiet | LOAD_NoWarn);
						if (FoundMesh)
						{
							ImportedObjects.Add(FoundMesh);
						}
					}
				}

				//Add the just imported object to the global array use for reimport test
				for (UObject* ImportObject : ImportedObjects)
				{
					GlobalImportedObjects.Add(ImportObject);
				}
			}
			break;
			case EFBXTestPlanActionType::Reimport:
			{
				if (GlobalImportedObjects.Num() < 1)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s: Cannot reimport when there is no previously imported object"), *CleanFilename));
					CurTestSuccessful = false;
					continue;
				}
				//Test expected result against the object we just reimport
				ImportedObjects.Add(GlobalImportedObjects[0]);

				if (GlobalImportedObjects[0]->IsA(UStaticMesh::StaticClass()))
				{
					UReimportFbxStaticMeshFactory* FbxStaticMeshReimportFactory = NewObject<UReimportFbxStaticMeshFactory>(UReimportFbxStaticMeshFactory::StaticClass());
					FbxStaticMeshReimportFactory->AddToRoot();
					
					FbxStaticMeshReimportFactory->ImportUI = TestPlan->ImportUI;

					UStaticMesh *ReimportStaticMesh = Cast<UStaticMesh>(GlobalImportedObjects[0]);
					UFbxStaticMeshImportData* ImportData = Cast<UFbxStaticMeshImportData>(ReimportStaticMesh->GetAssetImportData());

					//Copy UFbxStaticMeshImportData
					ImportData->StaticMeshLODGroup = TestPlan->ImportUI->StaticMeshImportData->StaticMeshLODGroup;
					ImportData->VertexColorImportOption = TestPlan->ImportUI->StaticMeshImportData->VertexColorImportOption;
					ImportData->VertexOverrideColor = TestPlan->ImportUI->StaticMeshImportData->VertexOverrideColor;
					ImportData->bRemoveDegenerates = TestPlan->ImportUI->StaticMeshImportData->bRemoveDegenerates;
					ImportData->bBuildReversedIndexBuffer = TestPlan->ImportUI->StaticMeshImportData->bBuildReversedIndexBuffer;
					ImportData->bGenerateLightmapUVs = TestPlan->ImportUI->StaticMeshImportData->bGenerateLightmapUVs;
					ImportData->bOneConvexHullPerUCX = TestPlan->ImportUI->StaticMeshImportData->bOneConvexHullPerUCX;
					ImportData->bAutoGenerateCollision = TestPlan->ImportUI->StaticMeshImportData->bAutoGenerateCollision;
					//Copy UFbxMeshImportData
					ImportData->bTransformVertexToAbsolute = TestPlan->ImportUI->StaticMeshImportData->bTransformVertexToAbsolute;
					ImportData->bBakePivotInVertex = TestPlan->ImportUI->StaticMeshImportData->bBakePivotInVertex;
					ImportData->bImportMeshLODs = TestPlan->ImportUI->StaticMeshImportData->bImportMeshLODs;
					ImportData->NormalImportMethod = TestPlan->ImportUI->StaticMeshImportData->NormalImportMethod;
					ImportData->NormalGenerationMethod = TestPlan->ImportUI->StaticMeshImportData->NormalGenerationMethod;
					ImportData->bComputeWeightedNormals = TestPlan->ImportUI->StaticMeshImportData->bComputeWeightedNormals;
					//Copy UFbxAssetImportData
					ImportData->ImportTranslation = TestPlan->ImportUI->StaticMeshImportData->ImportTranslation;
					ImportData->ImportRotation = TestPlan->ImportUI->StaticMeshImportData->ImportRotation;
					ImportData->ImportUniformScale = TestPlan->ImportUI->StaticMeshImportData->ImportUniformScale;
					ImportData->bImportAsScene = TestPlan->ImportUI->StaticMeshImportData->bImportAsScene;

					if (!FReimportManager::Instance()->Reimport(GlobalImportedObjects[0], false, false, CurFileToImport[0], FbxStaticMeshReimportFactory))
					{
						ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Error when reimporting the staticmesh"), *CleanFilename, *(TestPlan->TestPlanName)));
						CurTestSuccessful = false;
						continue;
					}
				}
				else if (GlobalImportedObjects[0]->IsA(USkeletalMesh::StaticClass()))
				{
					UReimportFbxSkeletalMeshFactory* FbxSkeletalMeshReimportFactory = NewObject<UReimportFbxSkeletalMeshFactory>(UReimportFbxSkeletalMeshFactory::StaticClass());
					FbxSkeletalMeshReimportFactory->AddToRoot();
					
					FbxSkeletalMeshReimportFactory->ImportUI = TestPlan->ImportUI;

					USkeletalMesh *ReimportSkeletalMesh = Cast<USkeletalMesh>(GlobalImportedObjects[0]);
					UFbxSkeletalMeshImportData* ImportData = Cast<UFbxSkeletalMeshImportData>(ReimportSkeletalMesh->GetAssetImportData());

					//Copy UFbxSkeletalMeshImportData
					ImportData->ImportContentType = TestPlan->ImportUI->SkeletalMeshImportData->ImportContentType;
					ImportData->bImportMeshesInBoneHierarchy = TestPlan->ImportUI->SkeletalMeshImportData->bImportMeshesInBoneHierarchy;
					ImportData->bImportMorphTargets = TestPlan->ImportUI->SkeletalMeshImportData->bImportMorphTargets;
					ImportData->bImportVertexAttributes = TestPlan->ImportUI->SkeletalMeshImportData->bImportVertexAttributes;
					ImportData->ThresholdPosition = TestPlan->ImportUI->SkeletalMeshImportData->ThresholdPosition;
					ImportData->ThresholdTangentNormal = TestPlan->ImportUI->SkeletalMeshImportData->ThresholdTangentNormal;
					ImportData->MorphThresholdPosition = TestPlan->ImportUI->SkeletalMeshImportData->MorphThresholdPosition;
					ImportData->ThresholdUV = TestPlan->ImportUI->SkeletalMeshImportData->ThresholdUV;
					ImportData->bPreserveSmoothingGroups = TestPlan->ImportUI->SkeletalMeshImportData->bPreserveSmoothingGroups;
					ImportData->bKeepSectionsSeparate = TestPlan->ImportUI->SkeletalMeshImportData->bKeepSectionsSeparate;
					ImportData->bUpdateSkeletonReferencePose = TestPlan->ImportUI->SkeletalMeshImportData->bUpdateSkeletonReferencePose;
					ImportData->bUseT0AsRefPose = TestPlan->ImportUI->SkeletalMeshImportData->bUseT0AsRefPose;
					//Copy UFbxMeshImportData
					ImportData->bTransformVertexToAbsolute = TestPlan->ImportUI->SkeletalMeshImportData->bTransformVertexToAbsolute;
					ImportData->bBakePivotInVertex = TestPlan->ImportUI->SkeletalMeshImportData->bBakePivotInVertex;
					ImportData->bImportMeshLODs = TestPlan->ImportUI->SkeletalMeshImportData->bImportMeshLODs;
					ImportData->NormalImportMethod = TestPlan->ImportUI->SkeletalMeshImportData->NormalImportMethod;
					ImportData->NormalGenerationMethod = TestPlan->ImportUI->SkeletalMeshImportData->NormalGenerationMethod;
					//Copy UFbxAssetImportData
					ImportData->ImportTranslation = TestPlan->ImportUI->SkeletalMeshImportData->ImportTranslation;
					ImportData->ImportRotation = TestPlan->ImportUI->SkeletalMeshImportData->ImportRotation;
					ImportData->ImportUniformScale = TestPlan->ImportUI->SkeletalMeshImportData->ImportUniformScale;
					ImportData->bImportAsScene = TestPlan->ImportUI->SkeletalMeshImportData->bImportAsScene;

					if (!FReimportManager::Instance()->Reimport(GlobalImportedObjects[0], false, false, CurFileToImport[0], FbxSkeletalMeshReimportFactory))
					{
						ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Error when reimporting the skeletal mesh"), *CleanFilename, *(TestPlan->TestPlanName)));
						CurTestSuccessful = false;
						FbxSkeletalMeshReimportFactory->RemoveFromRoot();
						continue;
					}
					FbxSkeletalMeshReimportFactory->RemoveFromRoot();
				}
			}
			break;
			case EFBXTestPlanActionType::AddLOD:
			case EFBXTestPlanActionType::ReimportLOD:
			{
				if (GlobalImportedObjects.Num() < 1)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s: Cannot reimport when there is no previously imported object"), *CleanFilename));
					CurTestSuccessful = false;
					continue;
				}

				//Test expected result against the object we just reimport
				ImportedObjects.Add(GlobalImportedObjects[0]);

				FString LodIndexString = FString::FromInt(TestPlan->LodIndex);
				if (TestPlan->LodIndex < 10)
				{
					LodIndexString = TEXT("0") + LodIndexString;
				}
				LodIndexString = TEXT("_lod") + LodIndexString;
				FString BaseLODFile = CurFileToImport[0];
				FString LodFile = BaseLODFile.Replace(TEXT("_lod00"), *LodIndexString);
				if (!FPaths::FileExists(LodFile))
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s: Cannot Add Lod because file %s do not exist on disk!"), *CleanFilename, *LodFile));
					CurTestSuccessful = false;
					continue;
				}
				constexpr bool bAsyncFalse = false;

				if (GlobalImportedObjects[0]->IsA(UStaticMesh::StaticClass()))
				{
					UStaticMesh *ExistingStaticMesh = Cast<UStaticMesh>(GlobalImportedObjects[0]);
					UFbxStaticMeshImportData* ImportData = Cast<UFbxStaticMeshImportData>(ExistingStaticMesh->GetAssetImportData());

					//Copy UFbxStaticMeshImportData
					ImportData->StaticMeshLODGroup = TestPlan->ImportUI->StaticMeshImportData->StaticMeshLODGroup;
					ImportData->VertexColorImportOption = TestPlan->ImportUI->StaticMeshImportData->VertexColorImportOption;
					ImportData->VertexOverrideColor = TestPlan->ImportUI->StaticMeshImportData->VertexOverrideColor;
					ImportData->bRemoveDegenerates = TestPlan->ImportUI->StaticMeshImportData->bRemoveDegenerates;
					ImportData->bBuildReversedIndexBuffer = TestPlan->ImportUI->StaticMeshImportData->bBuildReversedIndexBuffer;
					ImportData->bGenerateLightmapUVs = TestPlan->ImportUI->StaticMeshImportData->bGenerateLightmapUVs;
					ImportData->bOneConvexHullPerUCX = TestPlan->ImportUI->StaticMeshImportData->bOneConvexHullPerUCX;
					ImportData->bAutoGenerateCollision = TestPlan->ImportUI->StaticMeshImportData->bAutoGenerateCollision;
					//Copy UFbxMeshImportData
					ImportData->bTransformVertexToAbsolute = TestPlan->ImportUI->StaticMeshImportData->bTransformVertexToAbsolute;
					ImportData->bBakePivotInVertex = TestPlan->ImportUI->StaticMeshImportData->bBakePivotInVertex;
					ImportData->bImportMeshLODs = TestPlan->ImportUI->StaticMeshImportData->bImportMeshLODs;
					ImportData->NormalImportMethod = TestPlan->ImportUI->StaticMeshImportData->NormalImportMethod;
					ImportData->NormalGenerationMethod = TestPlan->ImportUI->StaticMeshImportData->NormalGenerationMethod;
					ImportData->bComputeWeightedNormals = TestPlan->ImportUI->StaticMeshImportData->bComputeWeightedNormals;
					//Copy UFbxAssetImportData
					ImportData->ImportTranslation = TestPlan->ImportUI->StaticMeshImportData->ImportTranslation;
					ImportData->ImportRotation = TestPlan->ImportUI->StaticMeshImportData->ImportRotation;
					ImportData->ImportUniformScale = TestPlan->ImportUI->StaticMeshImportData->ImportUniformScale;
					ImportData->bImportAsScene = TestPlan->ImportUI->StaticMeshImportData->bImportAsScene;

					FbxMeshUtils::ImportStaticMeshLOD(ExistingStaticMesh, LodFile, TestPlan->LodIndex, bAsyncFalse);
				}
				else if (GlobalImportedObjects[0]->IsA(USkeletalMesh::StaticClass()))
				{
					USkeletalMesh *ExistingSkeletalMesh = Cast<USkeletalMesh>(GlobalImportedObjects[0]);
					UFbxSkeletalMeshImportData* ImportData = Cast<UFbxSkeletalMeshImportData>(ExistingSkeletalMesh->GetAssetImportData());

					//Copy UFbxSkeletalMeshImportData
					ImportData->ImportContentType = TestPlan->ImportUI->SkeletalMeshImportData->ImportContentType;
					ImportData->bImportMeshesInBoneHierarchy = TestPlan->ImportUI->SkeletalMeshImportData->bImportMeshesInBoneHierarchy;
					ImportData->bImportMorphTargets = TestPlan->ImportUI->SkeletalMeshImportData->bImportMorphTargets;
					ImportData->bImportVertexAttributes = TestPlan->ImportUI->SkeletalMeshImportData->bImportVertexAttributes;
					ImportData->ThresholdPosition = TestPlan->ImportUI->SkeletalMeshImportData->ThresholdPosition;
					ImportData->ThresholdTangentNormal = TestPlan->ImportUI->SkeletalMeshImportData->ThresholdTangentNormal;
					ImportData->ThresholdUV = TestPlan->ImportUI->SkeletalMeshImportData->ThresholdUV;
					ImportData->MorphThresholdPosition = TestPlan->ImportUI->SkeletalMeshImportData->MorphThresholdPosition;
					ImportData->bPreserveSmoothingGroups = TestPlan->ImportUI->SkeletalMeshImportData->bPreserveSmoothingGroups;
					ImportData->bKeepSectionsSeparate = TestPlan->ImportUI->SkeletalMeshImportData->bKeepSectionsSeparate;
					ImportData->bUpdateSkeletonReferencePose = TestPlan->ImportUI->SkeletalMeshImportData->bUpdateSkeletonReferencePose;
					ImportData->bUseT0AsRefPose = TestPlan->ImportUI->SkeletalMeshImportData->bUseT0AsRefPose;
					//Copy UFbxMeshImportData
					ImportData->bTransformVertexToAbsolute = TestPlan->ImportUI->SkeletalMeshImportData->bTransformVertexToAbsolute;
					ImportData->bBakePivotInVertex = TestPlan->ImportUI->SkeletalMeshImportData->bBakePivotInVertex;
					ImportData->bImportMeshLODs = TestPlan->ImportUI->SkeletalMeshImportData->bImportMeshLODs;
					ImportData->NormalImportMethod = TestPlan->ImportUI->SkeletalMeshImportData->NormalImportMethod;
					ImportData->NormalGenerationMethod = TestPlan->ImportUI->SkeletalMeshImportData->NormalGenerationMethod;
					//Copy UFbxAssetImportData
					ImportData->ImportTranslation = TestPlan->ImportUI->SkeletalMeshImportData->ImportTranslation;
					ImportData->ImportRotation = TestPlan->ImportUI->SkeletalMeshImportData->ImportRotation;
					ImportData->ImportUniformScale = TestPlan->ImportUI->SkeletalMeshImportData->ImportUniformScale;
					ImportData->bImportAsScene = TestPlan->ImportUI->SkeletalMeshImportData->bImportAsScene;

					FbxMeshUtils::ImportSkeletalMeshLOD(ExistingSkeletalMesh, LodFile, TestPlan->LodIndex, bAsyncFalse);
				}
			}
			break;
			case EFBXTestPlanActionType::AddAlternateSkinnig:
			{
				if (GlobalImportedObjects.Num() < 1)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s: Cannot add alternate skinning when there is no previously imported object"), *CleanFilename));
					CurTestSuccessful = false;
					continue;
				}

				//Test expected result against the object we just import
				ImportedObjects.Add(GlobalImportedObjects[0]);

				FString ImportFilename = CurFileToImport[0];
				FString AltFile = ImportFilename.Replace(TEXT(".fbx"), TEXT("_alt.fbx"), ESearchCase::IgnoreCase);
				if (!FPaths::FileExists(AltFile))
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("Cannot add alternate skinning because file %s do not exist on disk!"), *AltFile));
					CurTestSuccessful = false;
					continue;
				}

				USkeletalMesh* ExistingSkeletalMesh = Cast<USkeletalMesh>(GlobalImportedObjects[0]);
				if (!ExistingSkeletalMesh)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s: Cannot add alternate skinning when there is no previously imported skeletal mesh."), *CleanFilename));
					CurTestSuccessful = false;
					continue;
				}

				{
					FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(ExistingSkeletalMesh);
					FScopedSkeletalMeshPostEditChange ScopedPostEditChange(ExistingSkeletalMesh);
					const FName ProfileName(TEXT("Alternate"));
					const int32 LodIndexZero = 0;
					const bool bResult = FSkinWeightsUtilities::ImportAlternateSkinWeight(ExistingSkeletalMesh, AltFile, LodIndexZero, ProfileName, false);
					if (!bResult)
					{
						ExecutionInfo.AddError(FString::Printf(TEXT("%s: Error adding alternate skinning file %s."), *CleanFilename, *AltFile));
						CurTestSuccessful = false;
						continue;
					}
				}
			}
			break;
		}

		//Garbage collect test options
		TestPlan->ImportUI->StaticMeshImportData->RemoveFromRoot();
		TestPlan->ImportUI->SkeletalMeshImportData->RemoveFromRoot();
		TestPlan->ImportUI->AnimSequenceImportData->RemoveFromRoot();
		TestPlan->ImportUI->TextureImportData->RemoveFromRoot();
		TestPlan->ImportUI->RemoveFromRoot();
		TestPlan->ImportUI = nullptr;
		TArray<FAssetData> ImportedAssets;
		AssetRegistryModule.Get().GetAssetsByPath(FName(*ImportAssetPath), ImportedAssets, true);

		WarningNum = ExecutionInfo.GetWarningTotal() - WarningNum;
		ErrorNum = ExecutionInfo.GetErrorTotal() - ErrorNum;
		int32 ExpectedResultIndex = 0;
		for (const FFbxTestPlanExpectedResult &ExpectedResult : TestPlan->ExpectedResult)
		{
			switch (ExpectedResult.ExpectedPresetsType)
			{
			case Error_Number:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 1)
				{
					ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("%s expected result need 1 integer data (Expected Error number)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Error_Number"), ExpectedResultIndex))));
					break;
				}
				if (ErrorNum != ExpectedResult.ExpectedPresetsDataInteger[0])
				{
					ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("%s [%d errors but expected %d]"),
						*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *( TestPlan->TestPlanName ), TEXT("Error_Number"), ExpectedResultIndex), ErrorNum, ExpectedResult.ExpectedPresetsDataInteger[0])));
				}
			}
			break;
			case Warning_Number:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 1)
				{
					ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("%s expected result need 1 integer data (Expected Warning number)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Warning_Number"), ExpectedResultIndex))));
					break;
				}

				if (WarningNum != ExpectedResult.ExpectedPresetsDataInteger[0])
				{
					ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("%s [%d warnings but expected %d]"),
						*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *( TestPlan->TestPlanName ), TEXT("Warning_Number"), ExpectedResultIndex), WarningNum, ExpectedResult.ExpectedPresetsDataInteger[0])));
				}
			}
			break;
			case Created_Staticmesh_Number:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 1)
				{
					ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("%s expected result need 1 integer data (Expected Static Mesh number)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Created_Staticmesh_Number"), ExpectedResultIndex))));
					break;
				}
				int32 StaticMeshImported = 0;
				for (UObject *ImportObject : ImportedObjects)
				{
					if (ImportObject->IsA(UStaticMesh::StaticClass()))
					{
						StaticMeshImported++;
					}
				}
				if (StaticMeshImported != ExpectedResult.ExpectedPresetsDataInteger[0])
				{
					ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("%s [%d staticmeshes created but expected %d]"),
						*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *( TestPlan->TestPlanName ), TEXT("Created_Staticmesh_Number"), ExpectedResultIndex), StaticMeshImported, ExpectedResult.ExpectedPresetsDataInteger[0])));
				}
			}
			break;
			case Created_Skeletalmesh_Number:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 1)
				{
					ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("%s expected result need 1 integer data (Expected Skeletal Mesh number)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Created_Skeletalmesh_Number"), ExpectedResultIndex))));
					break;
				}
				int32 SkeletalMeshImported = 0;
				for (UObject *ImportObject : ImportedObjects)
				{
					if (ImportObject->IsA(USkeletalMesh::StaticClass()))
					{
						SkeletalMeshImported++;
					}
				}
				if (SkeletalMeshImported != ExpectedResult.ExpectedPresetsDataInteger[0])
				{
					ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("%s [%d skeletalmeshes created but expected %d]"),
						*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *( TestPlan->TestPlanName ), TEXT("Created_Skeletalmesh_Number"), ExpectedResultIndex), SkeletalMeshImported, ExpectedResult.ExpectedPresetsDataInteger[0])));
				}
			}
			break;
			case Materials_Created_Number:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 1)
				{
					ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("%s expected result need 1 integer data (Expected Material number)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Materials_Created_Number"), ExpectedResultIndex))));
					break;
				}
				TArray<FAssetData> CreatedAssets;
				AssetRegistryModule.Get().GetAssetsByPath(FName(*PackagePath), CreatedAssets, true);
				int32 MaterialNumber = 0;
				for (FAssetData AssetData : CreatedAssets)
				{
					if (AssetData.AssetClassPath == UMaterial::StaticClass()->GetClassPathName() || AssetData.AssetClassPath == UMaterialInstanceConstant::StaticClass()->GetClassPathName())
					{
						MaterialNumber++;
					}
				}
				if (MaterialNumber != ExpectedResult.ExpectedPresetsDataInteger[0])
				{
					ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("%s [%d materials created but expected %d]"),
						*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *( TestPlan->TestPlanName ), TEXT("Materials_Created_Number"), ExpectedResultIndex), MaterialNumber, ExpectedResult.ExpectedPresetsDataInteger[0])));
				}
			}
			break;
			case Material_Slot_Imported_Name:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 1)
				{
					ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("%s expected result need 1 integer data (Expected material slot index)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Material_Slot_Imported_Name"), ExpectedResultIndex))));
					break;
				}
				if (ExpectedResult.ExpectedPresetsDataString.Num() < 1)
				{
					ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("%s expected result need 1 string data (Expected material imported name for the specified slot index)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Material_Slot_Imported_Name"), ExpectedResultIndex))));
					break;
				}
				int32 MeshMaterialNumber = INDEX_NONE;
				int32 MaterialSlotIndex = ExpectedResult.ExpectedPresetsDataInteger[0];
				FString ExpectedMaterialImportedName = ExpectedResult.ExpectedPresetsDataString[0];
				FString MaterialImportedName = TEXT("");
				bool BadSlotIndex = false;
				if (ImportedObjects.Num() > 0)
				{
					UObject *Object = ImportedObjects[0];
					if (Object->IsA(UStaticMesh::StaticClass()))
					{
						UStaticMesh *Mesh = Cast<UStaticMesh>(Object);
						if (Mesh->GetStaticMaterials().Num() <= MaterialSlotIndex)
						{
							BadSlotIndex = true;
							MeshMaterialNumber = Mesh->GetStaticMaterials().Num();
						}
						else
						{
							MaterialImportedName = Mesh->GetStaticMaterials()[MaterialSlotIndex].ImportedMaterialSlotName.ToString();
						}
					}
					else if (Object->IsA(USkeletalMesh::StaticClass()))
					{
						USkeletalMesh *Mesh = Cast<USkeletalMesh>(Object);
						if (Mesh->GetMaterials().Num() <= MaterialSlotIndex)
						{
							BadSlotIndex = true;
							MeshMaterialNumber = Mesh->GetMaterials().Num();
						}
						else
						{
							MaterialImportedName = Mesh->GetMaterials()[MaterialSlotIndex].ImportedMaterialSlotName.ToString();
						}
					}
				}
				if(BadSlotIndex)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Error in the test data, Material_Slot_Imported_Name material slot index [%d] is invalid. Expect something smaller then %d which is the mesh material number"),
						*CleanFilename, *(TestPlan->TestPlanName), MaterialSlotIndex, MeshMaterialNumber));
				}
				else if (MaterialImportedName != ExpectedMaterialImportedName)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s [Material slot index %d has a materials imported name %s but expected %s]"),
						*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *(TestPlan->TestPlanName), TEXT("Material_Slot_Imported_Name"), ExpectedResultIndex), MaterialSlotIndex, *MaterialImportedName, *ExpectedMaterialImportedName));
				}
			}
			break;
			case Vertex_Number:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 1)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 1 integer data (Expected Vertex number)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Vertex_Number"), ExpectedResultIndex)));
					break;
				}
				int32 GlobalVertexNumber = 0;
				if(ImportedObjects.Num() > 0)
				{
					UObject *Object = ImportedObjects[0];
					if (Object->IsA(UStaticMesh::StaticClass()))
					{
						UStaticMesh *StaticMesh = Cast<UStaticMesh>(Object);
						for (int32 LodIndex = 0; LodIndex < StaticMesh->GetNumLODs(); ++LodIndex)
						{
							GlobalVertexNumber += StaticMesh->GetNumVertices(LodIndex);
						}
					}
					else if (Object->IsA(USkeletalMesh::StaticClass()))
					{
						USkeletalMesh *SkeletalMesh = Cast<USkeletalMesh>(Object);
						for (int32 LodIndex = 0; LodIndex < SkeletalMesh->GetResourceForRendering()->LODRenderData.Num(); ++LodIndex)
						{
							GlobalVertexNumber += SkeletalMesh->GetResourceForRendering()->LODRenderData[LodIndex].GetNumVertices();
						}
					}
				}
				if (GlobalVertexNumber != ExpectedResult.ExpectedPresetsDataInteger[0])
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s [%d vertices but expected %d]"),
						*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *(TestPlan->TestPlanName), TEXT("Vertex_Number"), ExpectedResultIndex), GlobalVertexNumber, ExpectedResult.ExpectedPresetsDataInteger[0]));
				}
			}
			break;
			case Lod_Number:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 1)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 1 integer data (Expected LOD number)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Lod_Number"), ExpectedResultIndex)));
					break;
				}
				if (ImportedObjects.Num() > 0)
				{
					UObject *Object = ImportedObjects[0];
					int32 LodNumber = 0;
					if (Object->IsA(UStaticMesh::StaticClass()))
					{
						UStaticMesh *StaticMesh = Cast<UStaticMesh>(Object);
						LodNumber = StaticMesh->GetNumLODs();
					}
					else if (Object->IsA(USkeletalMesh::StaticClass()))
					{
						USkeletalMesh *SkeletalMesh = Cast<USkeletalMesh>(Object);
						LodNumber = SkeletalMesh->GetResourceForRendering()->LODRenderData.Num();
					}
					if (LodNumber != ExpectedResult.ExpectedPresetsDataInteger[0])
					{
						ExecutionInfo.AddError(FString::Printf(TEXT("%s [%d LODs but expected %d]"),
							*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *(TestPlan->TestPlanName), TEXT("Lod_Number"), ExpectedResultIndex), LodNumber, ExpectedResult.ExpectedPresetsDataInteger[0]));
					}
				}
				
			}
			break;
			case Vertex_Number_Lod:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 2)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 2 integer data (LOD index and Expected Vertex number for this LOD)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Vertex_Number_Lod"), ExpectedResultIndex)));
					break;
				}
				int32 LodIndex = ExpectedResult.ExpectedPresetsDataInteger[0];
				int32 GlobalVertexNumber = 0;
				if (ImportedObjects.Num() > 0)
				{
					UObject *Object = ImportedObjects[0];
					if (Object->IsA(UStaticMesh::StaticClass()))
					{
						UStaticMesh *StaticMesh = Cast<UStaticMesh>(Object);
						if(LodIndex < StaticMesh->GetNumLODs())
						{
							GlobalVertexNumber = StaticMesh->GetNumVertices(LodIndex);
						}
					}
					else if (Object->IsA(USkeletalMesh::StaticClass()))
					{
						USkeletalMesh *SkeletalMesh = Cast<USkeletalMesh>(Object);
						if (LodIndex < SkeletalMesh->GetResourceForRendering()->LODRenderData.Num())
						{
							GlobalVertexNumber = SkeletalMesh->GetResourceForRendering()->LODRenderData[LodIndex].GetNumVertices();
						}
					}
				}
				if (GlobalVertexNumber != ExpectedResult.ExpectedPresetsDataInteger[1])
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s [%d vertices but expected %d]"),
						*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *(TestPlan->TestPlanName), TEXT("Vertex_Number_Lod"), ExpectedResultIndex), GlobalVertexNumber, ExpectedResult.ExpectedPresetsDataInteger[1]));
				}
			}
			break;
			case Mesh_Materials_Number:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 1)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 1 integer data (Expected Material number)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Mesh_Materials_Number"), ExpectedResultIndex)));
					break;
				}
				int32 MaterialIndexNumber = -1;
				if (ImportedObjects.Num() > 0)
				{
					UObject *Object = ImportedObjects[0];
					if (Object->IsA(UStaticMesh::StaticClass()))
					{
						UStaticMesh *Mesh = Cast<UStaticMesh>(Object);
						MaterialIndexNumber = Mesh->GetStaticMaterials().Num();
					}
					else if (Object->IsA(USkeletalMesh::StaticClass()))
					{
						USkeletalMesh *Mesh = Cast<USkeletalMesh>(Object);
						MaterialIndexNumber = Mesh->GetMaterials().Num();
					}
				}
				if (MaterialIndexNumber != ExpectedResult.ExpectedPresetsDataInteger[0])
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s [%d materials indexes but expected %d]"),
						*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *(TestPlan->TestPlanName), TEXT("Mesh_Materials_Number"), ExpectedResultIndex), MaterialIndexNumber, ExpectedResult.ExpectedPresetsDataInteger[0]));
				}
			}
			break;
			case Mesh_LOD_Section_Number:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 2)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 2 integer data (LOD index and Expected sections number)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Mesh_LOD_Section_Number"), ExpectedResultIndex)));
					break;
				}
				int32 SectionNumber = -1;
				int32 LODIndex = ExpectedResult.ExpectedPresetsDataInteger[0];
				bool BadLodIndex = false;
				int32 LODNumber = 0;
				if (ImportedObjects.Num() > 0)
				{
					UObject *Object = ImportedObjects[0];
					if (Object->IsA(UStaticMesh::StaticClass()))
					{
						UStaticMesh *Mesh = Cast<UStaticMesh>(Object);
						LODNumber = Mesh->GetNumLODs();
						if (LODIndex < 0 || LODIndex >= LODNumber)
						{
							BadLodIndex = true;
						}
						else
						{
							SectionNumber = Mesh->GetRenderData()->LODResources[LODIndex].Sections.Num();
						}
					}
					else if (Object->IsA(USkeletalMesh::StaticClass()))
					{
						USkeletalMesh *Mesh = Cast<USkeletalMesh>(Object);
						LODNumber = Mesh->GetResourceForRendering()->LODRenderData.Num();
						if (LODIndex < 0 || LODIndex >= LODNumber)
						{
							BadLodIndex = true;
						}
						else
						{
							SectionNumber = Mesh->GetResourceForRendering()->LODRenderData[LODIndex].RenderSections.Num();
						}
					}
				}
				if (BadLodIndex)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Error in the test data, Mesh_LOD_Section_Number LOD index [%d] is invalid. Expect LODIndex between 0 and %d which is the mesh LOD number"),
						*CleanFilename, *(TestPlan->TestPlanName), LODIndex, LODNumber));
				}
				else if (SectionNumber != ExpectedResult.ExpectedPresetsDataInteger[1])
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s [LOD %d contain %d sections but expected %d section]"),
						*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *(TestPlan->TestPlanName), TEXT("Mesh_LOD_Section_Number"), ExpectedResultIndex), LODIndex, SectionNumber, ExpectedResult.ExpectedPresetsDataInteger[1]));
				}
			}
			break;
			case Mesh_LOD_Section_Vertex_Number:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 3)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 3 integer data (LOD index, section index and Expected vertex number)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Mesh_LOD_Section_Vertex_Number"), ExpectedResultIndex)));
					break;
				}
				int32 LODIndex = ExpectedResult.ExpectedPresetsDataInteger[0];
				int32 SectionIndex = ExpectedResult.ExpectedPresetsDataInteger[1];
				int32 ExpectedVertexNumber = ExpectedResult.ExpectedPresetsDataInteger[2];
				int32 LODNumber = 0;
				int32 SectionNumber = 0;
				bool BadLodIndex = false;
				bool BadSectionIndex = false;
				int32 SectionVertexNumber = 0;
				if (ImportedObjects.Num() > 0)
				{
					UObject *Object = ImportedObjects[0];
					if (Object->IsA(UStaticMesh::StaticClass()))
					{
						UStaticMesh *Mesh = Cast<UStaticMesh>(Object);
						LODNumber = Mesh->GetNumLODs();
						if (LODIndex < 0 || LODIndex >= LODNumber)
						{
							BadLodIndex = true;
						}
						else
						{
							SectionNumber = Mesh->GetRenderData()->LODResources[LODIndex].Sections.Num();
							if (SectionIndex < 0 || SectionIndex >= SectionNumber)
							{
								BadSectionIndex = true;
							}
							else
							{
								SectionVertexNumber = Mesh->GetRenderData()->LODResources[LODIndex].Sections[SectionIndex].NumTriangles * 3;
							}
						}
					}
					else if (Object->IsA(USkeletalMesh::StaticClass()))
					{
						USkeletalMesh *Mesh = Cast<USkeletalMesh>(Object);
						LODNumber = Mesh->GetResourceForRendering()->LODRenderData.Num();
						if (LODIndex < 0 || LODIndex >= LODNumber)
						{
							BadLodIndex = true;
						}
						else
						{
							SectionNumber = Mesh->GetResourceForRendering()->LODRenderData[LODIndex].RenderSections.Num();
							if (SectionIndex < 0 || SectionIndex >= SectionNumber)
							{
								BadSectionIndex = true;
							}
							else
							{
								SectionVertexNumber = Mesh->GetResourceForRendering()->LODRenderData[LODIndex].RenderSections[SectionIndex].GetNumVertices();
							}
						}
					}
				}
				if (BadLodIndex)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Error in the test data, Mesh_LOD_Section_Vertex_Number LOD index [%d] is invalid. Expect LODIndex between 0 and %d which is the mesh LOD number"),
						*CleanFilename, *(TestPlan->TestPlanName), LODIndex, LODNumber));
				}
				else if (BadSectionIndex)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Error in the test data, Mesh_LOD_Section_Vertex_Number Section index [%d] is invalid. Expect Section Index between 0 and %d which is the mesh LOD section number"),
						*CleanFilename, *(TestPlan->TestPlanName), LODIndex, SectionNumber));
				}
				else if (SectionVertexNumber != ExpectedVertexNumber)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s [LOD index %d Section index %d contain %d vertex but expected %d vertex]"),
						*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *(TestPlan->TestPlanName), TEXT("Mesh_LOD_Section_Vertex_Number"), ExpectedResultIndex), LODIndex, SectionIndex, SectionVertexNumber, ExpectedVertexNumber));
				}
			}
			break;
			case Mesh_LOD_Section_Triangle_Number:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 3)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 3 integer data (LOD index, section index and Expected triangle number)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Mesh_LOD_Section_Triangle_Number"), ExpectedResultIndex)));
					break;
				}
				int32 LODIndex = ExpectedResult.ExpectedPresetsDataInteger[0];
				int32 SectionIndex = ExpectedResult.ExpectedPresetsDataInteger[1];
				int32 ExpectedTriangleNumber = ExpectedResult.ExpectedPresetsDataInteger[2];
				int32 LODNumber = 0;
				int32 SectionNumber = 0;
				bool BadLodIndex = false;
				bool BadSectionIndex = false;
				int32 SectionTriangleNumber = 0;
				if (ImportedObjects.Num() > 0)
				{
					UObject *Object = ImportedObjects[0];
					if (Object->IsA(UStaticMesh::StaticClass()))
					{
						UStaticMesh *Mesh = Cast<UStaticMesh>(Object);
						LODNumber = Mesh->GetNumLODs();
						if (LODIndex < 0 || LODIndex >= LODNumber)
						{
							BadLodIndex = true;
						}
						else
						{
							SectionNumber = Mesh->GetRenderData()->LODResources[LODIndex].Sections.Num();
							if (SectionIndex < 0 || SectionIndex >= SectionNumber)
							{
								BadSectionIndex = true;
							}
							else
							{
								SectionTriangleNumber = Mesh->GetRenderData()->LODResources[LODIndex].Sections[SectionIndex].NumTriangles;
							}
						}
					}
					else if (Object->IsA(USkeletalMesh::StaticClass()))
					{
						USkeletalMesh *Mesh = Cast<USkeletalMesh>(Object);
						LODNumber = Mesh->GetResourceForRendering()->LODRenderData.Num();
						if (LODIndex < 0 || LODIndex >= LODNumber)
						{
							BadLodIndex = true;
						}
						else
						{
							SectionNumber = Mesh->GetResourceForRendering()->LODRenderData[LODIndex].RenderSections.Num();
							if (SectionIndex < 0 || SectionIndex >= SectionNumber)
							{
								BadSectionIndex = true;
							}
							else
							{
								SectionTriangleNumber = Mesh->GetResourceForRendering()->LODRenderData[LODIndex].RenderSections[SectionIndex].NumTriangles;
							}
						}
					}
				}
				if (BadLodIndex)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Error in the test data, Mesh_LOD_Section_Triangle_Number LOD index [%d] is invalid. Expect LODIndex between 0 and %d which is the mesh LOD number"),
						*CleanFilename, *(TestPlan->TestPlanName), LODIndex, LODNumber));
				}
				else if (BadSectionIndex)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Error in the test data, Mesh_LOD_Section_Triangle_Number Section index [%d] is invalid. Expect Section Index between 0 and %d which is the mesh LOD section number"),
						*CleanFilename, *(TestPlan->TestPlanName), LODIndex, SectionNumber));
				}
				else if (SectionTriangleNumber != ExpectedTriangleNumber)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s [LOD index %d Section index %d contain %d triangle but expected %d triangle]"),
						*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *(TestPlan->TestPlanName), TEXT("Mesh_LOD_Section_Triangle_Number"), ExpectedResultIndex), LODIndex, SectionIndex, SectionTriangleNumber, ExpectedTriangleNumber));
				}
			}
			break;
			case Mesh_LOD_Section_Material_Name:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 2 || ExpectedResult.ExpectedPresetsDataString.Num() < 1)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 2 integer data and 1 string(LOD index, section index and Expected material name)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Mesh_LOD_Section_Material_Name"), ExpectedResultIndex)));
					break;
				}
				int32 LODIndex = ExpectedResult.ExpectedPresetsDataInteger[0];
				int32 SectionIndex = ExpectedResult.ExpectedPresetsDataInteger[1];
				FString ExpectedMaterialName = ExpectedResult.ExpectedPresetsDataString[0];
				int32 LODNumber = 0;
				int32 SectionNumber = 0;
				bool BadLodIndex = false;
				bool BadSectionIndex = false;
				FString MaterialName;
				if (ImportedObjects.Num() > 0)
				{
					UObject *Object = ImportedObjects[0];
					if (Object->IsA(UStaticMesh::StaticClass()))
					{
						UStaticMesh *Mesh = Cast<UStaticMesh>(Object);
						LODNumber = Mesh->GetNumLODs();
						if (LODIndex < 0 || LODIndex >= LODNumber)
						{
							BadLodIndex = true;
						}
						else
						{
							SectionNumber = Mesh->GetRenderData()->LODResources[LODIndex].Sections.Num();
							if (SectionIndex < 0 || SectionIndex >= SectionNumber)
							{
								BadSectionIndex = true;
							}
							else
							{
								int32 MaterialIndex = Mesh->GetRenderData()->LODResources[LODIndex].Sections[SectionIndex].MaterialIndex;
								if (MaterialIndex >= 0 && MaterialIndex < Mesh->GetStaticMaterials().Num() && Mesh->GetStaticMaterials()[MaterialIndex].MaterialInterface != nullptr)
								{
									MaterialName = Mesh->GetStaticMaterials()[MaterialIndex].MaterialInterface->GetName();
								}
							}
						}
					}
					else if (Object->IsA(USkeletalMesh::StaticClass()))
					{
						USkeletalMesh *Mesh = Cast<USkeletalMesh>(Object);
						LODNumber = Mesh->GetResourceForRendering()->LODRenderData.Num();
						if (LODIndex < 0 || LODIndex >= LODNumber)
						{
							BadLodIndex = true;
						}
						else
						{
							SectionNumber = Mesh->GetResourceForRendering()->LODRenderData[LODIndex].RenderSections.Num();
							if (SectionIndex < 0 || SectionIndex >= SectionNumber)
							{
								BadSectionIndex = true;
							}
							else
							{
								int32 MaterialIndex = Mesh->GetResourceForRendering()->LODRenderData[LODIndex].RenderSections[SectionIndex].MaterialIndex;
								if (MaterialIndex >= 0 && MaterialIndex < Mesh->GetMaterials().Num())
								{
									MaterialName = Mesh->GetMaterials()[MaterialIndex].MaterialInterface->GetName();
								}
							}
						}
					}
				}
				if (BadLodIndex)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Error in the test data, Mesh_LOD_Section_Material_Name LOD index [%d] is invalid. Expect LODIndex between 0 and %d which is the mesh LOD number"),
						*CleanFilename, *(TestPlan->TestPlanName), LODIndex, LODNumber));
				}
				else if (BadSectionIndex)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Error in the test data, Mesh_LOD_Section_Material_Name Section index [%d] is invalid. Expect Section Index between 0 and %d which is the mesh LOD section number"),
						*CleanFilename, *(TestPlan->TestPlanName), LODIndex, SectionNumber));
				}
				else if (MaterialName.Compare(ExpectedMaterialName) != 0)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s [LOD index %d Section index %d contain material name (%s) but expected name (%s)]"),
						*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *(TestPlan->TestPlanName), TEXT("Mesh_LOD_Section_Material_Name"), ExpectedResultIndex), LODIndex, SectionIndex, *MaterialName, *ExpectedMaterialName));
				}
			}
			break;
			case Mesh_LOD_Section_Material_Index:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 2 || ExpectedResult.ExpectedPresetsDataString.Num() < 1)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 3 integer data (LOD index, section index and Expected material index)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Mesh_LOD_Section_Material_Index"), ExpectedResultIndex)));
					break;
				}
				int32 LODIndex = ExpectedResult.ExpectedPresetsDataInteger[0];
				int32 SectionIndex = ExpectedResult.ExpectedPresetsDataInteger[1];
				int32 ExpectedMaterialIndex = ExpectedResult.ExpectedPresetsDataInteger[2];
				int32 LODNumber = 0;
				int32 SectionNumber = 0;
				bool BadLodIndex = false;
				bool BadSectionIndex = false;
				int32 MaterialIndex = 0;
				if (ImportedObjects.Num() > 0)
				{
					UObject *Object = ImportedObjects[0];
					if (Object->IsA(UStaticMesh::StaticClass()))
					{
						UStaticMesh *Mesh = Cast<UStaticMesh>(Object);
						LODNumber = Mesh->GetNumLODs();
						if (LODIndex < 0 || LODIndex >= LODNumber)
						{
							BadLodIndex = true;
						}
						else
						{
							SectionNumber = Mesh->GetRenderData()->LODResources[LODIndex].Sections.Num();
							if (SectionIndex < 0 || SectionIndex >= SectionNumber)
							{
								BadSectionIndex = true;
							}
							else
							{
								MaterialIndex = Mesh->GetRenderData()->LODResources[LODIndex].Sections[SectionIndex].MaterialIndex;
							}
						}
					}
					else if (Object->IsA(USkeletalMesh::StaticClass()))
					{
						USkeletalMesh *Mesh = Cast<USkeletalMesh>(Object);
						LODNumber = Mesh->GetResourceForRendering()->LODRenderData.Num();
						if (LODIndex < 0 || LODIndex >= LODNumber)
						{
							BadLodIndex = true;
						}
						else
						{
							SectionNumber = Mesh->GetResourceForRendering()->LODRenderData[LODIndex].RenderSections.Num();
							if (SectionIndex < 0 || SectionIndex >= SectionNumber)
							{
								BadSectionIndex = true;
							}
							else
							{
								MaterialIndex = Mesh->GetResourceForRendering()->LODRenderData[LODIndex].RenderSections[SectionIndex].MaterialIndex;
							}
						}
					}
				}
				if (BadLodIndex)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Error in the test data, Mesh_LOD_Section_Material_Index LOD index [%d] is invalid. Expect LODIndex between 0 and %d which is the mesh LOD number"),
						*CleanFilename, *(TestPlan->TestPlanName), LODIndex, LODNumber));
				}
				else if (BadSectionIndex)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Error in the test data, Mesh_LOD_Section_Material_Index Section index [%d] is invalid. Expect Section Index between 0 and %d which is the mesh LOD section number"),
						*CleanFilename, *(TestPlan->TestPlanName), LODIndex, SectionNumber));
				}
				else if (MaterialIndex != ExpectedMaterialIndex)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s [LOD index %d Section index %d contain material index %d but expected index %d]"),
						*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *(TestPlan->TestPlanName), TEXT("Mesh_LOD_Section_Material_Index"), ExpectedResultIndex), LODIndex, SectionIndex, MaterialIndex, ExpectedMaterialIndex));
				}
			}
			break;
			case Mesh_LOD_Section_Material_Imported_Name:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 2 || ExpectedResult.ExpectedPresetsDataString.Num() < 1)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 2 integer data and 1 string(LOD index, section index and Expected material name)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Mesh_LOD_Section_Material_Imported_Name"), ExpectedResultIndex)));
					break;
				}
				int32 LODIndex = ExpectedResult.ExpectedPresetsDataInteger[0];
				int32 SectionIndex = ExpectedResult.ExpectedPresetsDataInteger[1];
				FString ExpectedMaterialName = ExpectedResult.ExpectedPresetsDataString[0];
				int32 LODNumber = 0;
				int32 SectionNumber = 0;
				bool BadLodIndex = false;
				bool BadSectionIndex = false;
				FString MaterialName;
				if (ImportedObjects.Num() > 0)
				{
					UObject *Object = ImportedObjects[0];
					if (Object->IsA(UStaticMesh::StaticClass()))
					{
						UStaticMesh *Mesh = Cast<UStaticMesh>(Object);
						LODNumber = Mesh->GetNumLODs();
						if (LODIndex < 0 || LODIndex >= LODNumber)
						{
							BadLodIndex = true;
						}
						else
						{
							SectionNumber = Mesh->GetRenderData()->LODResources[LODIndex].Sections.Num();
							if (SectionIndex < 0 || SectionIndex >= SectionNumber)
							{
								BadSectionIndex = true;
							}
							else
							{
								int32 MaterialIndex = Mesh->GetRenderData()->LODResources[LODIndex].Sections[SectionIndex].MaterialIndex;
								if (MaterialIndex >= 0 && MaterialIndex < Mesh->GetStaticMaterials().Num())
								{
									MaterialName = Mesh->GetStaticMaterials()[MaterialIndex].ImportedMaterialSlotName.ToString();
								}
							}
						}
					}
					else if (Object->IsA(USkeletalMesh::StaticClass()))
					{
						USkeletalMesh *Mesh = Cast<USkeletalMesh>(Object);
						LODNumber = Mesh->GetResourceForRendering()->LODRenderData.Num();
						if (LODIndex < 0 || LODIndex >= LODNumber)
						{
							BadLodIndex = true;
						}
						else
						{
							SectionNumber = Mesh->GetResourceForRendering()->LODRenderData[LODIndex].RenderSections.Num();
							if (SectionIndex < 0 || SectionIndex >= SectionNumber)
							{
								BadSectionIndex = true;
							}
							else
							{
								int32 MaterialIndex = Mesh->GetResourceForRendering()->LODRenderData[LODIndex].RenderSections[SectionIndex].MaterialIndex;
								if (MaterialIndex >= 0 && MaterialIndex < Mesh->GetMaterials().Num())
								{
									MaterialName = Mesh->GetMaterials()[MaterialIndex].ImportedMaterialSlotName.ToString();
								}
							}
						}
					}
				}
				if (BadLodIndex)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Error in the test data, Mesh_LOD_Section_Material_Imported_Name LOD index [%d] is invalid. Expect LODIndex between 0 and %d which is the mesh LOD number"),
						*CleanFilename, *(TestPlan->TestPlanName), LODIndex, LODNumber));
				}
				else if (BadSectionIndex)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Error in the test data, Mesh_LOD_Section_Material_Imported_Name Section index [%d] is invalid. Expect Section Index between 0 and %d which is the mesh LOD section number"),
						*CleanFilename, *(TestPlan->TestPlanName), LODIndex, SectionNumber));
				}
				else if (MaterialName.Compare(ExpectedMaterialName) != 0)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s [LOD index %d Section index %d contain import material name (%s) but expected name (%s)]"),
						*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *(TestPlan->TestPlanName), TEXT("Mesh_LOD_Section_Material_Imported_Name"), ExpectedResultIndex), LODIndex, SectionIndex, *MaterialName, *ExpectedMaterialName));
				}
			}
			break;

			case Mesh_LOD_Vertex_Position:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 2)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 2 integer data (LOD index, vertex index)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Mesh_LOD_Vertex_Position"), ExpectedResultIndex)));
					break;
				}
				if (ExpectedResult.ExpectedPresetsDataFloat.Num() < 3)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 3 float data (expected position X, Y and Z)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Mesh_LOD_Vertex_Position"), ExpectedResultIndex)));
					break;
				}
				const int32 LODIndex = ExpectedResult.ExpectedPresetsDataInteger[0];
				const int32 VertexIndex = ExpectedResult.ExpectedPresetsDataInteger[1];
				const FVector ExpectedPosition(ExpectedResult.ExpectedPresetsDataFloat[0], ExpectedResult.ExpectedPresetsDataFloat[1], ExpectedResult.ExpectedPresetsDataFloat[2]);
				int32 LODNumber = 0;
				int32 VertexNumber = 0;
				bool BadLodIndex = false;
				bool BadVertexIndex = false;
				FVector VertexPosition(0.0f);
				if (ImportedObjects.Num() > 0)
				{
					UObject *Object = ImportedObjects[0];
					if (Object->IsA(UStaticMesh::StaticClass()))
					{
						UStaticMesh *Mesh = Cast<UStaticMesh>(Object);
						LODNumber = Mesh->GetNumLODs();
						if (LODIndex < 0 || LODIndex >= LODNumber)
						{
							BadLodIndex = true;
						}
						else
						{
							VertexNumber = Mesh->GetRenderData()->LODResources[LODIndex].VertexBuffers.PositionVertexBuffer.GetNumVertices();
							if (VertexIndex < 0 || VertexIndex >= VertexNumber)
							{
								BadVertexIndex = true;
							}
							else
							{
								VertexPosition = (FVector)Mesh->GetRenderData()->LODResources[LODIndex].VertexBuffers.PositionVertexBuffer.VertexPosition(VertexIndex);
							}
						}
					}
					else if (Object->IsA(USkeletalMesh::StaticClass()))
					{
						USkeletalMesh *Mesh = Cast<USkeletalMesh>(Object);
						LODNumber = Mesh->GetResourceForRendering()->LODRenderData.Num();
						if (LODIndex < 0 || LODIndex >= LODNumber)
						{
							BadLodIndex = true;
						}
						else
						{
							VertexNumber = Mesh->GetResourceForRendering()->LODRenderData[LODIndex].StaticVertexBuffers.PositionVertexBuffer.GetNumVertices();
							if (VertexIndex < 0 || VertexIndex >= VertexNumber)
							{
								BadVertexIndex = true;
							}
							else
							{
								VertexPosition = (FVector)Mesh->GetResourceForRendering()->LODRenderData[LODIndex].StaticVertexBuffers.PositionVertexBuffer.VertexPosition(VertexIndex);
							}
						}
					}
				}
				if (BadLodIndex)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Error in the test data, Mesh_LOD_Vertex_Position LOD index [%d] is invalid. Expect LODIndex between 0 and %d which is the mesh LOD number"),
						*CleanFilename, *(TestPlan->TestPlanName), LODIndex, LODNumber));
				}
				else if (BadVertexIndex)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Error in the test data, Mesh_LOD_Vertex_Position Vertex index [%d] is invalid. Expect Vertex Index between 0 and %d which is the mesh LOD vertex number"),
						*CleanFilename, *(TestPlan->TestPlanName), VertexIndex, VertexNumber));
				}
				else if (!VertexPosition.Equals(ExpectedPosition))
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s [LOD index %d Vertex index %d has the following position (%s) but expected position (%s)]"),
						*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *(TestPlan->TestPlanName), TEXT("Mesh_LOD_Vertex_Position"), ExpectedResultIndex), LODIndex, VertexIndex, *VertexPosition.ToString(), *ExpectedPosition.ToString()));
				}
			}
			break;

			case Mesh_LOD_Vertex_Normal:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 2)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 2 integer data (LOD index, vertex index)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Mesh_LOD_Vertex_Normal"), ExpectedResultIndex)));
					break;
				}
				if (ExpectedResult.ExpectedPresetsDataFloat.Num() < 3)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 3 float data (expected normal X, Y and Z)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Mesh_LOD_Vertex_Normal"), ExpectedResultIndex)));
					break;
				}
				const int32 LODIndex = ExpectedResult.ExpectedPresetsDataInteger[0];
				const int32 VertexIndex = ExpectedResult.ExpectedPresetsDataInteger[1];
				const FVector3f ExpectedNormal(ExpectedResult.ExpectedPresetsDataFloat[0], ExpectedResult.ExpectedPresetsDataFloat[1], ExpectedResult.ExpectedPresetsDataFloat[2]);
				int32 LODNumber = 0;
				int32 VertexNumber = 0;
				bool BadLodIndex = false;
				bool BadVertexIndex = false;
				FVector3f VertexNormal(0.0f);
				if (ImportedObjects.Num() > 0)
				{
					UObject *Object = ImportedObjects[0];
					if (Object->IsA(UStaticMesh::StaticClass()))
					{
						UStaticMesh *Mesh = Cast<UStaticMesh>(Object);
						LODNumber = Mesh->GetNumLODs();
						if (LODIndex < 0 || LODIndex >= LODNumber)
						{
							BadLodIndex = true;
						}
						else
						{
							VertexNumber = Mesh->GetRenderData()->LODResources[LODIndex].VertexBuffers.StaticMeshVertexBuffer.GetNumVertices();
							if (VertexIndex < 0 || VertexIndex >= VertexNumber)
							{
								BadVertexIndex = true;
							}
							else
							{
								VertexNormal = Mesh->GetRenderData()->LODResources[LODIndex].VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(VertexIndex);
							}
						}
					}
					else if (Object->IsA(USkeletalMesh::StaticClass()))
					{
						USkeletalMesh *Mesh = Cast<USkeletalMesh>(Object);
						LODNumber = Mesh->GetResourceForRendering()->LODRenderData.Num();
						if (LODIndex < 0 || LODIndex >= LODNumber)
						{
							BadLodIndex = true;
						}
						else
						{
							VertexNumber = Mesh->GetResourceForRendering()->LODRenderData[LODIndex].StaticVertexBuffers.StaticMeshVertexBuffer.GetNumVertices();
							if (VertexIndex < 0 || VertexIndex >= VertexNumber)
							{
								BadVertexIndex = true;
							}
							else
							{
								VertexNormal = Mesh->GetResourceForRendering()->LODRenderData[LODIndex].StaticVertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(VertexIndex);
							}
						}
					}
				}
				if (BadLodIndex)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Error in the test data, Mesh_LOD_Vertex_Normal LOD index [%d] is invalid. Expect LODIndex between 0 and %d which is the mesh LOD number"),
						*CleanFilename, *(TestPlan->TestPlanName), LODIndex, LODNumber));
				}
				else if (BadVertexIndex)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Error in the test data, Mesh_LOD_Vertex_Normal Vertex index [%d] is invalid. Expect Vertex Index between 0 and %d which is the mesh LOD vertex number"),
						*CleanFilename, *(TestPlan->TestPlanName), VertexIndex, VertexNumber));
				}
				else if (!VertexNormal.Equals(ExpectedNormal))
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s [LOD index %d Vertex index %d has the following normal (%s) but expected normal (%s)]"),
						*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *(TestPlan->TestPlanName), TEXT("Mesh_LOD_Vertex_Normal"), ExpectedResultIndex), LODIndex, VertexIndex, *VertexNormal.ToString(), *ExpectedNormal.ToString()));
				}
			}
			break;

			case LOD_UV_Channel_Number:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 2)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 2 integer data (LOD index and Expected UV Channel number)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("LOD_UV_Channel_Number"), ExpectedResultIndex)));
					break;
				}
				int32 UVChannelNumber = -1;
				int32 LODIndex = ExpectedResult.ExpectedPresetsDataInteger[0];
				int32 ExpectedUVNumber = ExpectedResult.ExpectedPresetsDataInteger[1];
				int32 LODNumber = -1;
				bool BadLodIndex = false;
				if (ImportedObjects.Num() > 0)
				{
					UObject *Object = ImportedObjects[0];
					if (Object->IsA(UStaticMesh::StaticClass()))
					{
						UStaticMesh *Mesh = Cast<UStaticMesh>(Object);
						LODNumber = Mesh->GetNumLODs();
						if (LODIndex < 0 || LODIndex >= LODNumber)
						{
							BadLodIndex = true;
						}
						else
						{
							UVChannelNumber = Mesh->GetRenderData()->LODResources[LODIndex].GetNumTexCoords();
						}
					}
					else if (Object->IsA(USkeletalMesh::StaticClass()))
					{
						USkeletalMesh *Mesh = Cast<USkeletalMesh>(Object);
						LODNumber = Mesh->GetResourceForRendering()->LODRenderData.Num();
						if (LODIndex < 0 || LODIndex >= LODNumber)
						{
							BadLodIndex = true;
						}
						else
						{
							UVChannelNumber = Mesh->GetResourceForRendering()->LODRenderData[LODIndex].GetNumTexCoords();
						}
					}
				}
				if (BadLodIndex)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Error in the test data, LOD_UV_Channel_Number LOD index [%d] is invalid. Expect LODIndex between 0 and %d which is the mesh LOD number"),
						*CleanFilename, *(TestPlan->TestPlanName), LODIndex, LODNumber));
				}
				else if (UVChannelNumber != ExpectedUVNumber)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s [%d UVChannels but expected %d]"),
						*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *(TestPlan->TestPlanName), TEXT("LOD_UV_Channel_Number"), ExpectedResultIndex), UVChannelNumber, ExpectedUVNumber));
				}
			}
			break;
			case Bone_Number:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 1)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 1 integer data (Expected Bone number)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Bone_Number"), ExpectedResultIndex)));
					break;
				}
				int32 BoneNumber = -1;
				if (ImportedObjects.Num() > 0)
				{
					UObject *Object = ImportedObjects[0];
					if (Object->IsA(USkeletalMesh::StaticClass()))
					{
						USkeletalMesh *Mesh = Cast<USkeletalMesh>(Object);
						if (Mesh->GetSkeleton() != nullptr)
						{
							BoneNumber = Mesh->GetSkeleton()->GetReferenceSkeleton().GetNum();
						}
					}
				}
				if (BoneNumber != ExpectedResult.ExpectedPresetsDataInteger[0])
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s [%d bones but expected %d]"),
						*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *(TestPlan->TestPlanName), TEXT("Bone_Number"), ExpectedResultIndex), BoneNumber, ExpectedResult.ExpectedPresetsDataInteger[0]));
				}
			}
			break;
			case Bone_Position:
			{
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 1 || ExpectedResult.ExpectedPresetsDataFloat.Num() < 3)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 1 integer data and 3 float data (Bone index and expected bone position XYZ)"),
						*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Bone_Position"), ExpectedResultIndex)));
					break;
				}
				int32 BoneIndex = ExpectedResult.ExpectedPresetsDataInteger[0];
				FVector ExpectedBonePosition(ExpectedResult.ExpectedPresetsDataFloat[0], ExpectedResult.ExpectedPresetsDataFloat[1], ExpectedResult.ExpectedPresetsDataFloat[2]);
				float Epsilon = ExpectedResult.ExpectedPresetsDataFloat.Num() == 4 ? ExpectedResult.ExpectedPresetsDataFloat[3] : FLT_EPSILON;
				FVector BoneIndexPosition(FLT_MAX);
				bool FoundSkeletalMesh = false;
				bool FoundSkeleton = false;
				bool FoundBoneIndex = false;
				int32 BoneNumber = -1;
				if (ImportedObjects.Num() > 0)
				{
					UObject *Object = ImportedObjects[0];
					if (Object->IsA(USkeletalMesh::StaticClass()))
					{
						FoundSkeletalMesh = true;
						USkeletalMesh *Mesh = Cast<USkeletalMesh>(Object);
						FoundSkeleton = true;
						BoneNumber = Mesh->GetRefSkeleton().GetNum();
						if (BoneIndex >= 0 && BoneIndex < BoneNumber)
						{
							FoundBoneIndex = true;
							BoneIndexPosition = Mesh->GetRefSkeleton().GetRefBonePose()[BoneIndex].GetLocation();
						}
					}
				}
				if (!FoundSkeletalMesh)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Wrong Expected Result, there is no skeletal mesh imported"),
						*CleanFilename, *(TestPlan->TestPlanName)));
				}
				else if (!FoundSkeleton)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Wrong Expected Result, there is no skeleton attach to this skeletal mesh"),
						*CleanFilename, *(TestPlan->TestPlanName)));
				}
				else if (!FoundBoneIndex)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Wrong Expected Result, the bone index is not a valid index (bone index [%d] bone number[%d])"),
						*CleanFilename, *(TestPlan->TestPlanName), BoneIndex, BoneNumber));
				}
				if (!BoneIndexPosition.Equals(ExpectedBonePosition, Epsilon))
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s [X:%f, Y:%f, Z:%f but expected X:%f, Y:%f, Z:%f]"),
						*GetFormatedMessageErrorInExpectedResult(*CleanFilename, *(TestPlan->TestPlanName), TEXT("Bone_Position"), ExpectedResultIndex), BoneIndexPosition.X, BoneIndexPosition.Y, BoneIndexPosition.Z, ExpectedBonePosition.X, ExpectedBonePosition.Y, ExpectedBonePosition.Z));
				}
			}
			break;
			
			case Animation_Frame_Number:
			{
				FString FormatedMessageErrorPrefix = GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Animation_Frame_Number"), ExpectedResultIndex);
				if (UAnimSequence* AnimSequence = FFbxImportAssetsAutomationTestHelper::GetImportedAnimSequence(ImportedAssets, ExecutionInfo, FormatedMessageErrorPrefix))
				{
					if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 1)
					{
						ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 1 integer data (Expected Animation Frame Number)"),
							*FormatedMessageErrorPrefix));
						break;
					}
					int32 FrameNumber = AnimSequence->GetNumberOfSampledKeys();
					if (FrameNumber != ExpectedResult.ExpectedPresetsDataInteger[0])
					{
						ExecutionInfo.AddError(FString::Printf(TEXT("%s [%d frames but expected %d]"),
							*FormatedMessageErrorPrefix, FrameNumber, ExpectedResult.ExpectedPresetsDataInteger[0]));
					}
				}
			}
			break;
			
			case Animation_Length:
			{
				FString FormatedMessageErrorPrefix = GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Animation_Length"), ExpectedResultIndex);
				if (UAnimSequence* AnimSequence = FFbxImportAssetsAutomationTestHelper::GetImportedAnimSequence(ImportedAssets, ExecutionInfo, FormatedMessageErrorPrefix))
				{
					if (ExpectedResult.ExpectedPresetsDataFloat.Num() < 1)
					{
						ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 1 float data (Expected Animation Length in seconds)"),
							*FormatedMessageErrorPrefix));
						break;
					}
					float AnimationLength = AnimSequence->GetPlayLength();
					if (!FMath::IsNearlyEqual(AnimationLength, ExpectedResult.ExpectedPresetsDataFloat[0], 0.001f))
					{
						ExecutionInfo.AddError(FString::Printf(TEXT("%s [%f seconds but expected %f]"),
							*FormatedMessageErrorPrefix, AnimationLength, ExpectedResult.ExpectedPresetsDataFloat[0]));
					}
				}
			}
			break;
			case Animation_CustomCurve_KeyValue:
			{
				FString FormatedMessageErrorPrefix = GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Animation_CustomCurve_KeyValue"), ExpectedResultIndex);
				FRichCurveKey CustomCurveKey;
				if (FFbxImportAssetsAutomationTestHelper::GetImportedCustomCurveKey(ImportedAssets, ExecutionInfo, FormatedMessageErrorPrefix, ExpectedResult, CustomCurveKey))
				{
					if (ExpectedResult.ExpectedPresetsDataFloat.Num() < 1)
					{
						ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 1 float data (Expected Custom Curve Key value)"),
							*FormatedMessageErrorPrefix));
						break;
					}

					const float KeyValue = CustomCurveKey.Value;
					if (!FMath::IsNearlyEqual(KeyValue, ExpectedResult.ExpectedPresetsDataFloat[0], 0.001f))
					{
						ExecutionInfo.AddError(FString::Printf(TEXT("%s the value for the specified key [%f] does not match the expected value [%f]"),
							*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Animation_CustomCurve_KeyValue"), ExpectedResultIndex), KeyValue, ExpectedResult.ExpectedPresetsDataFloat[0]));

						break;
					}
				}
				break;
			}
			case Animation_CustomCurve_KeyArriveTangent:
			{
				FString FormatedMessageErrorPrefix = GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Animation_CustomCurve_KeyArriveTangent"), ExpectedResultIndex);
				FRichCurveKey CustomCurveKey;
				if (FFbxImportAssetsAutomationTestHelper::GetImportedCustomCurveKey(ImportedAssets, ExecutionInfo, FormatedMessageErrorPrefix, ExpectedResult, CustomCurveKey))
				{
					if (ExpectedResult.ExpectedPresetsDataFloat.Num() < 1)
					{
						ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 1 float data (Expected Custom Curve Key Arriving Tangent value)"),
							*FormatedMessageErrorPrefix));
						break;
					}

					const float ArriveTangent = CustomCurveKey.ArriveTangent;
					if (!FMath::IsNearlyEqual(ArriveTangent, ExpectedResult.ExpectedPresetsDataFloat[0], 0.001f))
					{
						ExecutionInfo.AddError(FString::Printf(TEXT("%s the value for the specified arriving tangent [%f] does not match the expected value [%f]"),
							*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Animation_CustomCurve_KeyArriveTangent"), ExpectedResultIndex), ArriveTangent, ExpectedResult.ExpectedPresetsDataFloat[0]));

						break;
					}
				}
				break;
			}
			case Animation_CustomCurve_KeyLeaveTangent:
			{
				FString FormatedMessageErrorPrefix = GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Animation_CustomCurve_KeyLeaveTangent"), ExpectedResultIndex);
				FRichCurveKey CustomCurveKey;
				if (FFbxImportAssetsAutomationTestHelper::GetImportedCustomCurveKey(ImportedAssets, ExecutionInfo, FormatedMessageErrorPrefix, ExpectedResult, CustomCurveKey))
				{
					if (ExpectedResult.ExpectedPresetsDataFloat.Num() < 1)
					{
						ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 1 float data (Expected Custom Curve Key Leaving Tangent value)"),
							*FormatedMessageErrorPrefix));
						break;
					}

					const float LeaveTangent = CustomCurveKey.LeaveTangent;
					if (!FMath::IsNearlyEqual(LeaveTangent, ExpectedResult.ExpectedPresetsDataFloat[0], 0.001f))
					{
						ExecutionInfo.AddError(FString::Printf(TEXT("%s the value for the specified leaving tangent [%f] does not match the expected value [%f]"),
							*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Animation_CustomCurve_KeyLeaveTangent"), ExpectedResultIndex), LeaveTangent, ExpectedResult.ExpectedPresetsDataFloat[0]));

						break;
					}
				}
				break;
			}
			case Animation_CustomCurve_KeyArriveTangentWeight:
			{
				FString FormatedMessageErrorPrefix = GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Animation_CustomCurve_KeyArriveTangentWeight"), ExpectedResultIndex);
				FRichCurveKey CustomCurveKey;
				if (FFbxImportAssetsAutomationTestHelper::GetImportedCustomCurveKey(ImportedAssets, ExecutionInfo, FormatedMessageErrorPrefix, ExpectedResult, CustomCurveKey))
				{
					if (ExpectedResult.ExpectedPresetsDataFloat.Num() < 1)
					{
						ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 1 float data (Expected Custom Curve Key Arriving Tangent Weight value)"),
							*FormatedMessageErrorPrefix));
						break;
					}

					const float ArriveTangentWeight = CustomCurveKey.ArriveTangentWeight;
					if (!FMath::IsNearlyEqual(ArriveTangentWeight, ExpectedResult.ExpectedPresetsDataFloat[0], 0.001f))
					{
						ExecutionInfo.AddError(FString::Printf(TEXT("%s the value for the specified arriving tangent weight [%f] does not match the expected value [%f]"),
							*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Animation_CustomCurve_KeyArriveTangentWeight"), ExpectedResultIndex), ArriveTangentWeight, ExpectedResult.ExpectedPresetsDataFloat[0]));

						break;
					}
				}
				break;
			}
			case Animation_CustomCurve_KeyLeaveTangentWeight:
			{
				FString FormatedMessageErrorPrefix = GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Animation_CustomCurve_KeyLeaveTangentWeight"), ExpectedResultIndex);
				FRichCurveKey CustomCurveKey;
				if (FFbxImportAssetsAutomationTestHelper::GetImportedCustomCurveKey(ImportedAssets, ExecutionInfo, FormatedMessageErrorPrefix, ExpectedResult, CustomCurveKey))
				{
					if (ExpectedResult.ExpectedPresetsDataFloat.Num() < 1)
					{
						ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 1 float data (Expected Custom Curve Key Leaving Tangent Weight value)"),
							*FormatedMessageErrorPrefix));
						break;
					}

					const float LeaveTangentWeight = CustomCurveKey.LeaveTangentWeight;
					if (!FMath::IsNearlyEqual(LeaveTangentWeight, ExpectedResult.ExpectedPresetsDataFloat[0], 0.001f))
					{
						ExecutionInfo.AddError(FString::Printf(TEXT("%s the value for the specified leaving tangent weight [%f] does not match the expected value [%f]"),
							*GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Animation_CustomCurve_KeyLeaveTangentWeight"), ExpectedResultIndex), LeaveTangentWeight, ExpectedResult.ExpectedPresetsDataFloat[0]));

						break;
					}
				}
				break;
			}
			case Skin_By_Bone_Vertex_Number:
			{
				if (ExpectedResult.ExpectedPresetsDataString.Num() < 1)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 1 string data for the bone name"),
										   *GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Skin_By_Bone_Vertex_Number"), ExpectedResultIndex)));
					break;
				}
				if (ExpectedResult.ExpectedPresetsDataInteger.Num() < 2)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s expected result need 2 integer data: [0] specify if we test alternate profile (0 no, 1 yes). [1] is the expected vertex number skin by the specified bone)"),
										   *GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Skin_By_Bone_Vertex_Number"), ExpectedResultIndex)));
					break;
				}
				bool bInspectProfile = ExpectedResult.ExpectedPresetsDataInteger[0] != 0;
				FString BoneName = ExpectedResult.ExpectedPresetsDataString[0];
				int32 ExpectedVertexSkinByBoneNumber = ExpectedResult.ExpectedPresetsDataInteger[1];
				int32 VertexSkinByBoneNumber = 0;
				bool FoundSkeletalMesh = false;
				bool FoundBoneIndex = false;
				bool bFoundVertexCount = false;
				bool bFoundProfile = false;
				if (ImportedObjects.Num() > 0)
				{
					UObject* Object = ImportedObjects[0];
					if (Object->IsA(USkeletalMesh::StaticClass()))
					{
						FoundSkeletalMesh = true;
						USkeletalMesh* Mesh = Cast<USkeletalMesh>(Object);
						int32 BoneIndex = Mesh->GetRefSkeleton().FindBoneIndex(*BoneName);
						if (Mesh->GetRefSkeleton().IsValidIndex(BoneIndex))
						{
							FoundBoneIndex = true;
							if (Mesh->GetImportedModel() && Mesh->GetImportedModel()->LODModels.IsValidIndex(0))
							{
								auto IncrementInfluence = [&VertexSkinByBoneNumber, &BoneIndex](const FSkelMeshSection& Section, const FBoneIndexType (&InfluenceBones)[MAX_TOTAL_INFLUENCES], const uint16 (&InfluenceWeights)[MAX_TOTAL_INFLUENCES])
								{
									for (int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; ++InfluenceIndex)
									{
										if (InfluenceWeights[InfluenceIndex] == 0)
										{
											//Influence are sorted by weight so no need to go further then a zero weight
											break;
										}
										if (Section.BoneMap[InfluenceBones[InfluenceIndex]] == BoneIndex)
										{
											VertexSkinByBoneNumber++;
											break;
										}
									}
								};

								if (!bInspectProfile) //Inspect the base skinning
								{
									for (const FSkelMeshSection& Section : Mesh->GetImportedModel()->LODModels[0].Sections)
									{
										const int32 SectionVertexCount = Section.SoftVertices.Num();
										//Find the number of vertex skin by this bone
										for (int32 SectionVertexIndex = 0; SectionVertexIndex < SectionVertexCount; ++SectionVertexIndex)
										{
											const FSoftSkinVertex& Vertex = Section.SoftVertices[SectionVertexIndex];
											IncrementInfluence(Section, Vertex.InfluenceBones, Vertex.InfluenceWeights);
										}
									}
								}
								else //Inspect the first alternate profile
								{
									if (Mesh->GetSkinWeightProfiles().Num() > 0)
									{
										const int32 TotalVertexCount = Mesh->GetImportedModel()->LODModels[0].NumVertices;
										const FSkinWeightProfileInfo& SkinWeightProfile = Mesh->GetSkinWeightProfiles()[0];
										const FImportedSkinWeightProfileData& SkinWeightData = Mesh->GetImportedModel()->LODModels[0].SkinWeightProfiles.FindChecked(SkinWeightProfile.Name);
										bFoundProfile = SkinWeightData.SkinWeights.Num() == TotalVertexCount;
										int32 TotalVertexIndex = 0;
										for (const FSkelMeshSection& Section : Mesh->GetImportedModel()->LODModels[0].Sections)
										{
											const int32 SectionVertexCount = Section.SoftVertices.Num();
											//Find the number of vertex skin by this bone
											for (int32 SectionVertexIndex = 0; SectionVertexIndex < SectionVertexCount; ++SectionVertexIndex, ++TotalVertexIndex)
											{
												const FRawSkinWeight& SkinWeight = SkinWeightData.SkinWeights[TotalVertexIndex];
												IncrementInfluence(Section, SkinWeight.InfluenceBones, SkinWeight.InfluenceWeights);
											}
										}
									}
								}
								bFoundVertexCount = true;
							}
						}
					}
				}
				if (!FoundSkeletalMesh)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Wrong Expected Result, there is no skeletal mesh imported"),
										   *CleanFilename, *(TestPlan->TestPlanName)));
				}
				else if (!FoundBoneIndex)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Wrong Expected Result, the bone name is not an existing (bone name [%s])"),
										   *CleanFilename, *(TestPlan->TestPlanName), *BoneName));
				}
				else if (bInspectProfile && !bFoundProfile)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s Cannot find alternate skinning profile, data argument specify to inspect skinning profile."),
										   *GetFormatedMessageErrorInTestData(CleanFilename, TestPlan->TestPlanName, TEXT("Skin_By_Bone_Vertex_Number"), ExpectedResultIndex)));
					break;
				}
				else if (!bFoundVertexCount)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Wrong Expected Result, there is no valid mesh geometry to find the vertex count."),
										   *CleanFilename, *(TestPlan->TestPlanName)));
				}
				if (VertexSkinByBoneNumber != ExpectedVertexSkinByBoneNumber)
				{
					ExecutionInfo.AddError(FString::Printf(TEXT("%s [%d vertex skin by bone %s, but expected %d]"),
										   *GetFormatedMessageErrorInExpectedResult(*CleanFilename, *(TestPlan->TestPlanName), TEXT("Skin_By_Bone_Vertex_Number"), ExpectedResultIndex)
										   , VertexSkinByBoneNumber, *BoneName, ExpectedVertexSkinByBoneNumber));
				}
				break;
			}
			default:
			{
				ExecutionInfo.AddError(FString::Printf(TEXT("%s->%s: Wrong Test plan, Unknown expected result preset."),
					*CleanFilename, *(TestPlan->TestPlanName)));
			}
			break;
			};
			ExpectedResultIndex++;
		}
		if (TestPlan->bDeleteFolderAssets || TestPlan->Action == EFBXTestPlanActionType::ImportReload)
		{
			if ((GEditor != nullptr) && (GEditor->Trans != nullptr))
			{
				GEditor->Trans->Reset(FText::FromString("Discard undo history during FBX Automation testing."));
			}

			//When doing an import-reload we have to destroy the package since it was save
			//But when we just have everything in memory a garbage collection pass is enough to
			//delete assets.
			if (TestPlan->Action != EFBXTestPlanActionType::ImportReload)
			{
				for (const FAssetData& AssetData : ImportedAssets)
				{
					UPackage* Package = AssetData.GetPackage();
					if (Package != nullptr)
					{
						for (FThreadSafeObjectIterator It; It; ++It)
						{
							UObject* ExistingObject = *It;
							if ((ExistingObject->GetOutermost() == Package))
							{
								ExistingObject->ClearFlags(RF_Standalone | RF_Public);
								ExistingObject->RemoveFromRoot();
								ExistingObject->MarkAsGarbage();
							}

						}
					}
				}
				CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
			}

			//Make sure there is no more asset under "Engine\Content\FbxEditorAutomationOut" folder
			GlobalImportedObjects.Empty();
			TArray<FAssetData> AssetsToDelete;
			AssetRegistryModule.Get().GetAssetsByPath(FName(*PackagePath), AssetsToDelete, true);
			const bool bShowConfirmation = false;
			TArray<UObject*> ObjectToDelete;
			for (const FAssetData& AssetData : AssetsToDelete)
			{
				UObject *Asset = AssetData.GetAsset();
				if (Asset != nullptr)
					ObjectToDelete.Add(Asset);
			}
			ObjectTools::ForceDeleteObjects(ObjectToDelete, bShowConfirmation);
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		}
	}
	
	return CurTestSuccessful;
}

END_FUNCTION_BUILD_OPTIMIZATION