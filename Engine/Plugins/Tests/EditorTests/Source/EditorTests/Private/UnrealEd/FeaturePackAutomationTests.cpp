// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/AutomationTest.h"
#include "FeaturePackContentSource.h"

DEFINE_LOG_CATEGORY_STATIC(FeaturePackAutomationTestLog, Log, All);

// 
// namespace FeaturePackAutomationTests
// {
// }

/**
 * Feature Pack validity test
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFeaturePackValidityTest, "Editor.Content.FeaturePackValidityTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

/** 
 * Scans feature pack folder and verifies that all feature packs therein are parsed successfully.
 *
 * @param Parameters - Unused for this test
 * @return	TRUE if the test was successful, FALSE otherwise
 */
bool FFeaturePackValidityTest::RunTest(const FString& Parameters)
{
	FString FeaturePackPath = FPaths::FeaturePackDir();
	FeaturePackPath += TEXT("*.upack");
	int32 FailCount = 0;
	IPlatformFile &PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	
	TArray<FString> FeaturePacks;
	IFileManager::Get().FindFiles(FeaturePacks, *FeaturePackPath, true, false );

	for (auto FeaturePackFile : FeaturePacks)
	{
		TUniquePtr<FFeaturePackContentSource> NewContentSource = MakeUnique<FFeaturePackContentSource>(FPaths::FeaturePackDir() + FeaturePackFile);
		if (NewContentSource->IsDataValid() == false)
		{
			FailCount++;
			for (int32 iError = 0; iError < NewContentSource->ParseErrors.Num() ; iError++)
			{
				AddError(NewContentSource->ParseErrors[iError]);
			}
			
		}
	}
	
	return FailCount==0;
}
