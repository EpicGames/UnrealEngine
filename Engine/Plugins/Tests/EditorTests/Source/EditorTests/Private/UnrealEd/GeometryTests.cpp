// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Engine/Brush.h"
#include "EngineUtils.h"
#include "Engine/BrushBuilder.h"
#include "Builders/ConeBuilder.h"
#include "Builders/CubeBuilder.h"
#include "Builders/CurvedStairBuilder.h"
#include "Builders/CylinderBuilder.h"
#include "Builders/LinearStairBuilder.h"
#include "Builders/SheetBuilder.h"
#include "Builders/SpiralStairBuilder.h"
#include "Builders/TetrahedronBuilder.h"
#include "Builders/VolumetricBuilder.h"
#include "EditorModeManager.h"

//Automation
#include "Model.h"
#include "Tests/AutomationCommon.h"
#include "Tests/AutomationEditorCommon.h"

#define CUBE_VERTS_COUNT			8
#define WORLD_ORIGIN_VECTOR			FVector(0.f,0.f,0.f)
#define WORLD_ORIGIN_VECTOR_OFFSET	FVector(0.f,0.f,10.f)

DEFINE_LOG_CATEGORY_STATIC(LogGeometryTests, All, All);


namespace GeometryPromotionHelpers
{
	void MiscPreparationsForBrushCreation(const ULevel* InCurrentLevel, int32& CurrentLevelVerts, const FString& InBrushType)
	{
		UE_LOGF(LogGeometryTests, Log, "Adding an '%ls' brush.", *InBrushType);
		CurrentLevelVerts = InCurrentLevel->Model->Verts.Num();
	}

	/**
	* Checks that the level geometry changed after adding a brush and checks that undo / redo works
	*
	* @param CurrentLevel - The current level
	* @param VertsBefore - The number of verts before the brush was added
	* @param VertsAfter - The number of verts after the brush was added
	* @param BrushType - The type of brush to use in the log
	* @param bAdditive - True if this brush was added in additive mode
	*/
	void TestGeometryUndoRedo(ULevel* CurrentLevel, int32 VertsBefore, int32 VertsAfter, const FString& BrushType, bool bAdditive)
	{
		if (VertsAfter > VertsBefore)
		{
			UE_LOGF(LogEditorAutomationTests, Log, "Placed a %ls in %ls mode", *BrushType, bAdditive ? TEXT("additive") : TEXT("subtractive"));
		}
		else
		{
			UE_LOGF(LogEditorAutomationTests, Error, "Failed to place a %ls in %ls mode", *BrushType, bAdditive ? TEXT("additive") : TEXT("subtractive"));
		}

		//Undo
		GEditor->UndoTransaction();
		if (CurrentLevel->Model->Verts.Num() == VertsBefore)
		{
			UE_LOGF(LogEditorAutomationTests, Log, "Undo successful for %ls", *BrushType);
		}
		else
		{
			UE_LOGF(LogEditorAutomationTests, Error, "Undo failed for %ls", *BrushType);
		}

		//Redo
		GEditor->RedoTransaction();
		if (CurrentLevel->Model->Verts.Num() == VertsAfter)
		{
			UE_LOGF(LogEditorAutomationTests, Log, "Redo successful for %ls", *BrushType);
		}
		else
		{
			UE_LOGF(LogEditorAutomationTests, Error, "Redo failed for %ls", *BrushType);
		}
	}
}


//////////////////////////////////////////////////////////////////////////
//BSP Promotion Test
//////////////////////////////////////////////////////////////////////////

/**
* BSP Promotion Test
* Adds one of every brush type to the world.  Most of which will be set as additive brushes.
* Undo and redo the placement of an additive and subtractive brush.
* There will be a few subtractive brushes intersecting with other objects.
* Tests for number of surfaces and vertices, numbers of brushes created, and brush location.
*/
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGeometryPromotionValidation, "System.Promotion.Editor.Geometry Validation", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FGeometryPromotionValidation::RunTest(const FString& Parameters)
{
	//** SETUP **//
	// Set the Test Description
	FString Description = TEXT("Geometry Validation:\n- Adds one of every brush type to the world.\n- Undo and redo the placement of an additive and subtractive brush.\n- Verify by finding the number of surfaces, vertices, brushes created, and their location.\n");
	AddInfo(Description);

	// Create the world and setup the level.
	UWorld* World = FAutomationEditorCommonUtils::CreateNewMap();
	GLevelEditorModeTools().MapChangeNotify();
	ULevel* CurrentLevel = World->GetCurrentLevel();
	// Set the view location so that it can see the entire scene.
	FAutomationEditorCommonUtils::SetOrthoViewportView(FVector(176, 2625, 2075), FRotator(319, 269, 1));
	// Grabbing the current number of verts in the world to compare later on.
	int32 VertsBefore = CurrentLevel->Model->Verts.Num();
	GEditor->Exec(World, TEXT("BRUSH Scale 1 1 1"));

	//** TEST **//
	//Cube Additive Brush
	UE_LOGF(LogGeometryTests, Log, "Adding an 'Additive Cube' brush.");
	UCubeBuilder* CubeAdditiveBrushBuilder = Cast<UCubeBuilder>(GEditor->FindBrushBuilder(UCubeBuilder::StaticClass()));
	CubeAdditiveBrushBuilder->X = 4096.0f;
	CubeAdditiveBrushBuilder->Y = 4096.0f;
	CubeAdditiveBrushBuilder->Z = 128.0f;
	CubeAdditiveBrushBuilder->Build(World);
	GEditor->Exec(World, TEXT("BRUSH MOVETO X=0 Y=0 Z=0"));
	GEditor->Exec(World, TEXT("BRUSH ADD"));
	// Test that Undo/Redo works for an additive brush. 
	int32 VertsAfter = CurrentLevel->Model->Verts.Num();
	GeometryPromotionHelpers::TestGeometryUndoRedo(CurrentLevel, VertsBefore, VertsAfter, TEXT("Cube"), true);

	//Cone Additive Brush
	GeometryPromotionHelpers::MiscPreparationsForBrushCreation(CurrentLevel, VertsBefore, TEXT("Additive Cone"));
	UConeBuilder* ConeAdditiveBrushBuilder = Cast<UConeBuilder>(GEditor->FindBrushBuilder(UConeBuilder::StaticClass()));
	ConeAdditiveBrushBuilder->Z = 1024.0f;
	ConeAdditiveBrushBuilder->CapZ = 256.0f;
	ConeAdditiveBrushBuilder->OuterRadius = 512.0f;
	ConeAdditiveBrushBuilder->InnerRadius = 384;
	ConeAdditiveBrushBuilder->Sides = 32;
	ConeAdditiveBrushBuilder->Build(World);
	GEditor->Exec(World, TEXT("BRUSH MOVETO X=-1525 Y=-1777 Z=64"));
	GEditor->Exec(World, TEXT("BRUSH ADD"));

	//Sphere Additive Brush
	GeometryPromotionHelpers::MiscPreparationsForBrushCreation(CurrentLevel, VertsBefore, TEXT("Additive Sphere"));
	UTetrahedronBuilder* TetraAdditiveBrushBuilder = Cast<UTetrahedronBuilder>(GEditor->FindBrushBuilder(UTetrahedronBuilder::StaticClass()));
	TetraAdditiveBrushBuilder->Radius = 512.0f;
	TetraAdditiveBrushBuilder->SphereExtrapolation = 3;
	TetraAdditiveBrushBuilder->Build(World);
	GEditor->Exec(World, TEXT("BRUSH MOVETO X=-88 Y=-1777 Z=535"));
	GEditor->Exec(World, TEXT("BRUSH ADD"));

	//Cylinder Additive Brush
	GeometryPromotionHelpers::MiscPreparationsForBrushCreation(CurrentLevel, VertsBefore, TEXT("Additive Cylinder"));
	UCylinderBuilder* CylinderAdditiveBrushBuilder = Cast<UCylinderBuilder>(GEditor->FindBrushBuilder(UCylinderBuilder::StaticClass()));
	CylinderAdditiveBrushBuilder->Z = 1024.0f;
	CylinderAdditiveBrushBuilder->OuterRadius = 512.0f;
	CylinderAdditiveBrushBuilder->InnerRadius = 384.0f;
	CylinderAdditiveBrushBuilder->Sides = 16;
	CylinderAdditiveBrushBuilder->Build(World);
	GEditor->Exec(World, TEXT("BRUSH MOVETO X=1338 Y=-1776 Z=535"));
	GEditor->Exec(World, TEXT("BRUSH ADD"));

	//Sheet Additive Brush
	GeometryPromotionHelpers::MiscPreparationsForBrushCreation(CurrentLevel, VertsBefore, TEXT("Additive Sheet"));
	USheetBuilder* SheetAdditiveBrushBuilder = Cast<USheetBuilder>(GEditor->FindBrushBuilder(USheetBuilder::StaticClass()));
	SheetAdditiveBrushBuilder->X = 512.0f;
	SheetAdditiveBrushBuilder->Y = 512.0f;
	SheetAdditiveBrushBuilder->XSegments = 1;
	SheetAdditiveBrushBuilder->YSegments = 1;
	SheetAdditiveBrushBuilder->Axis = AX_YAxis;
	SheetAdditiveBrushBuilder->Build(World);
	GEditor->Exec(World, TEXT("BRUSH MOVETO X=-760 Y=-346 Z=535"));
	GEditor->Exec(World, TEXT("BRUSH ADD"));

	//Volumetric Additive Brush
	GeometryPromotionHelpers::MiscPreparationsForBrushCreation(CurrentLevel, VertsBefore, TEXT("Additive Volumetric"));
	UVolumetricBuilder* VolumetricAdditiveBrushBuilder = Cast<UVolumetricBuilder>(GEditor->FindBrushBuilder(UVolumetricBuilder::StaticClass()));
	VolumetricAdditiveBrushBuilder->Z = 512.0f;
	VolumetricAdditiveBrushBuilder->Radius = 128.0f;
	VolumetricAdditiveBrushBuilder->NumSheets = 3;
	VolumetricAdditiveBrushBuilder->Build(World);
	GEditor->Exec(World, TEXT("BRUSH MOVETO X=445 Y=-345 Z=535"));
	GEditor->Exec(World, TEXT("BRUSH ADD"));

	//Linear Stair Additive Brush
	GeometryPromotionHelpers::MiscPreparationsForBrushCreation(CurrentLevel, VertsBefore, TEXT("Additive Linear Stair"));
	ULinearStairBuilder* LinearStairAdditiveBrushBuilder = Cast<ULinearStairBuilder>(GEditor->FindBrushBuilder(ULinearStairBuilder::StaticClass()));
	LinearStairAdditiveBrushBuilder->StepLength = 64.0f;
	LinearStairAdditiveBrushBuilder->StepHeight = 16.0f;
	LinearStairAdditiveBrushBuilder->StepWidth = 256.0f;
	LinearStairAdditiveBrushBuilder->NumSteps = 8;
	LinearStairAdditiveBrushBuilder->AddToFirstStep = 0;
	LinearStairAdditiveBrushBuilder->Build(World);
	GEditor->Exec(World, TEXT("BRUSH MOVETO X=1464 Y=-345 Z=-61"));
	GEditor->Exec(World, TEXT("BRUSH ADD"));

	//Curved Stair Additive Brush
	GeometryPromotionHelpers::MiscPreparationsForBrushCreation(CurrentLevel, VertsBefore, TEXT("Additive Curved Stair"));
	UCurvedStairBuilder* CurvedStairAdditiveBrushBuilder = Cast<UCurvedStairBuilder>(GEditor->FindBrushBuilder(UCurvedStairBuilder::StaticClass()));
	CurvedStairAdditiveBrushBuilder->InnerRadius = 240.0f;
	CurvedStairAdditiveBrushBuilder->StepHeight = 16.0f;
	CurvedStairAdditiveBrushBuilder->StepWidth = 256.0f;
	CurvedStairAdditiveBrushBuilder->AngleOfCurve = 90.0f;
	CurvedStairAdditiveBrushBuilder->NumSteps = 4;
	CurvedStairAdditiveBrushBuilder->AddToFirstStep = 0;
	CurvedStairAdditiveBrushBuilder->CounterClockwise = false;
	CurvedStairAdditiveBrushBuilder->Build(World);
	GEditor->Exec(World, TEXT("BRUSH MOVETO X=-1290 Y=263 Z=193"));
	GEditor->Exec(World, TEXT("BRUSH ADD"));

	//Spiral Stair Additive Brush
	GeometryPromotionHelpers::MiscPreparationsForBrushCreation(CurrentLevel, VertsBefore, TEXT("Additive Spiral Stair"));
	USpiralStairBuilder* SpiralStairAdditiveBrushBuilder = Cast<USpiralStairBuilder>(GEditor->FindBrushBuilder(USpiralStairBuilder::StaticClass()));
	SpiralStairAdditiveBrushBuilder->InnerRadius = 64;
	SpiralStairAdditiveBrushBuilder->StepWidth = 256.0f;
	SpiralStairAdditiveBrushBuilder->StepHeight = 16.0f;
	SpiralStairAdditiveBrushBuilder->StepThickness = 32.0f;
	SpiralStairAdditiveBrushBuilder->NumStepsPer360 = 8;
	SpiralStairAdditiveBrushBuilder->NumSteps = 8;
	SpiralStairAdditiveBrushBuilder->SlopedCeiling = true;
	SpiralStairAdditiveBrushBuilder->Build(World);
	GEditor->Exec(World, TEXT("BRUSH MOVETO X=850 Y=263 Z=193"));
	GEditor->Exec(World, TEXT("BRUSH ADD"));

	//Cylinder Subtractive Brush
	GeometryPromotionHelpers::MiscPreparationsForBrushCreation(CurrentLevel, VertsBefore, TEXT("Subtractive Cylinder"));
	UCylinderBuilder* CylinderSubtractiveBrushBuilder = Cast<UCylinderBuilder>(GEditor->FindBrushBuilder(UCylinderBuilder::StaticClass()));
	CylinderSubtractiveBrushBuilder->Z = 256;
	CylinderSubtractiveBrushBuilder->OuterRadius = 512.0f;
	CylinderSubtractiveBrushBuilder->InnerRadius = 384.0f;
	CylinderSubtractiveBrushBuilder->Sides = 3;
	CylinderSubtractiveBrushBuilder->Build(World);
	GEditor->Exec(World, TEXT("BRUSH MOVETO X=0 Y=0 Z=128"));
	GEditor->Exec(World, TEXT("BRUSH SUBTRACT"));
	// Test that Undo/Redo works for a subtractive brush.  
	VertsAfter = CurrentLevel->Model->Verts.Num();
	GeometryPromotionHelpers::TestGeometryUndoRedo(CurrentLevel, VertsBefore, VertsAfter, TEXT("Cylinder"), false);

	//Cube Subtractive Brush
	GeometryPromotionHelpers::MiscPreparationsForBrushCreation(CurrentLevel, VertsBefore, TEXT("Subtractive Cube"));
	UCubeBuilder* CubeSubtractiveBrushBuilder = Cast<UCubeBuilder>(GEditor->FindBrushBuilder(UCubeBuilder::StaticClass()));
	CubeSubtractiveBrushBuilder->X = 256.0f;
	CubeSubtractiveBrushBuilder->Y = 1024.0f;
	CubeSubtractiveBrushBuilder->Z = 256.0f;
	CubeSubtractiveBrushBuilder->Build(World);
	GEditor->Exec(World, TEXT("BRUSH MOVETO X=-88 Y=-1777 Z=535"));
	GEditor->Exec(World, TEXT("BRUSH SUBTRACT"));


	//** VERIFY **//
	// Gather the objects needed to run the tests.
	// Get the levels BSP Model.
	UModel* BSPModel = CurrentLevel->Model;
	// Add the brush actors to an array.
	TArray<ABrush*> AdditiveBSP;
	TArray<ABrush*> SubtractiveBSP;
	for (TActorIterator<ABrush> It(World); It; ++It)
	{
		ABrush* FoundBrush = *It;

		if (FoundBrush->BrushType == EBrushType::Brush_Add)
		{
			AdditiveBSP.Add(FoundBrush);
		}
		else if (FoundBrush->BrushType == EBrushType::Brush_Subtract)
		{
			SubtractiveBSP.Add(FoundBrush);
		}
	}

	// Verify the BSP brushes that were created have been found.
	TestEqual(TEXT("Additive BSP Brushes Created"), AdditiveBSP.Num(), 9);
	TestEqual(TEXT("Subtractive BSP Brushes Created"), SubtractiveBSP.Num(), 2);

	// Verify there are the correct amount of BSP surfaces are visible in the level.
	TestEqual(TEXT("Surfaces Reported"), BSPModel->Surfs.Num(), 276);
	UE_LOGF(LogGeometryTests, Log, "The number of BSP Brushes Found in the level: Additive: %i, Subtractive: %i", AdditiveBSP.Num(), SubtractiveBSP.Num());
	
	//** TEARDOWN **//
	AdditiveBSP.Empty();
	SubtractiveBSP.Empty();

	return true;
}


//////////////////////////////////////////////////////////////////////////
//BSP Unit Tests
//////////////////////////////////////////////////////////////////////////

/**
* BSP Cube - New Brush tests
* Adds two new BSP cubes to the world. An Additive Brush and a Subtractive Brush.
* Subtractive brush intersects will the additive brush.
* Tests for number of surfaces and vertices, numbers of brushes created, and brush location.
*/
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBSPCubePlacement, "Editor.Geometry.BSP Cube.Brush Placement", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FBSPCubePlacement::RunTest(const FString& Parameters)
{
	//** SETUP **//
	// Create the world and setup the level.
	UWorld* World = FAutomationEditorCommonUtils::CreateNewMap();
	GLevelEditorModeTools().MapChangeNotify();
	ULevel* CurrentLevel = World->GetCurrentLevel();
	UBrushBuilder* DefaultBuilderBrush = World->GetDefaultBrush()->BrushBuilder;

	//** TEST **//
	// Add a new additive cube BSP to the world using it's default settings.
	GEditor->Exec(World, TEXT("BRUSH Scale 1 1 1"));
	World->GetDefaultBrush()->BrushBuilder = Cast<UCubeBuilder>(GEditor->FindBrushBuilder(UCubeBuilder::StaticClass()));
	GEditor->Exec(World, TEXT("BRUSH MOVETO X=0 Y=0 Z=10"));
	GEditor->Exec(World, TEXT("BRUSH ADD"));

	// Add a new subtractive cube BSP to the world.  It will be interesting with the previous cube. 
	// The cubes settings are being altered.
	UCubeBuilder* CubeAdditiveBrushBuilder = Cast<UCubeBuilder>(GEditor->FindBrushBuilder(UCubeBuilder::StaticClass()));
	CubeAdditiveBrushBuilder->X = 300.0f;
	CubeAdditiveBrushBuilder->Y = 100.0f;
	CubeAdditiveBrushBuilder->Z = 100.0f;
	GEditor->Exec(World, TEXT("BRUSH Scale 1 1 1"));
	CubeAdditiveBrushBuilder->Build(World);
	GEditor->Exec(World, TEXT("BRUSH MOVETO X=0 Y=0 Z=0"));
	GEditor->Exec(World, TEXT("BRUSH SUBTRACT"));

	//** VERIFY **//
	// Gather the objects needed to run the tests.
	// Get the levels BSP Model.
	UModel* BSPModel = CurrentLevel->Model;
	// Set the brush actor variables to the generated additive and subtractive brushes.
	ABrush* AdditiveBSP = nullptr;
	ABrush* SubtractiveBSP = nullptr;
	int32 BrushCount = 0;
	for ( TActorIterator<ABrush> It(World); It; ++It )
	{
		ABrush* FoundBrush = *It;

		if (FoundBrush->BrushType == EBrushType::Brush_Add)
		{
			AdditiveBSP = FoundBrush;
			BrushCount++;
		}
		else if (FoundBrush->BrushType == EBrushType::Brush_Subtract)
		{
			SubtractiveBSP = FoundBrush;
			BrushCount++;
		}
	}

	// Verify the BSP brushes that were created have been found.
	TestEqual(TEXT("BSP Brushes Created"), BrushCount, 2);

	TestNotNull<ABrush>(TEXT("Additive BSP is NULL.  Most likely it wasn't created."), AdditiveBSP);

	// Verify the location of the BSP.
	if (AdditiveBSP)
	{
		TestEqual<FVector>(TEXT("Additive BSP is not located at the correct coordinates."), AdditiveBSP->GetActorLocation(), WORLD_ORIGIN_VECTOR_OFFSET);
	}
	if (SubtractiveBSP)
	{
		TestEqual<FVector>(TEXT("Subtractive BSP is not located at the correct coordinates."), SubtractiveBSP->GetActorLocation(), WORLD_ORIGIN_VECTOR);
	}

	// Verify there are 8 vertices for the Additive and Subtractive BSP.
	if (AdditiveBSP)
	{
		TestEqual(TEXT("Additive Brush Vertex Count"), AdditiveBSP->Brush->NumUniqueVertices, CUBE_VERTS_COUNT);
	}
	if (SubtractiveBSP)
	{
		TestEqual(TEXT("Subtractive Brush Vertex Count"), SubtractiveBSP->Brush->NumUniqueVertices, CUBE_VERTS_COUNT);
	}
	

	
	// Verify there are the correct amount of BSP surfaces are visible in the level.
	TestEqual(TEXT("Surfaces Reported"), BSPModel->Surfs.Num(), 10);

#if WITH_EDITOR
	// Check the BSP for any other errors.
	if (SubtractiveBSP)
	{
		SubtractiveBSP->CheckForErrors();
	}
	if (AdditiveBSP)
	{
		AdditiveBSP->CheckForErrors();
	}
#endif // WITH_EDITOR

	//** TEARDOWN **//
	// Set the default builder brush back to its original state.
	World->GetDefaultBrush()->BrushBuilder = DefaultBuilderBrush;

	return true;
}


/**
* BSP Cube - Undo and Redo
* Adds an additive brush to the world and then does an undo/redo of that transaction.
* Tests for number vertices in the level.
*/
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBSPCubeUndoRedoPlacement, "Editor.Geometry.BSP Cube.Brush Placement Undo Redo", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FBSPCubeUndoRedoPlacement::RunTest(const FString& Parameters)
{
	//** SETUP **//
	// Create the world and setup the level.
	UWorld* World = FAutomationEditorCommonUtils::CreateNewMap();
	ULevel* CurrentLevel = World->GetCurrentLevel();
	GLevelEditorModeTools().MapChangeNotify();
	GEditor->Exec(World, TEXT("BRUSH Scale 1 1 1"));
	
	int32 DefaultVertCount = CurrentLevel->Model->Verts.Num();

	// Add a cube brush to the world.
	UCubeBuilder* CubeAdditiveBrushBuilder = Cast<UCubeBuilder>(GEditor->FindBrushBuilder(UCubeBuilder::StaticClass()));
	CubeAdditiveBrushBuilder->Build(World);
	GEditor->Exec(World, TEXT("BRUSH MOVETO X=0 Y=0 Z=0"));
	GEditor->Exec(World, TEXT("BRUSH ADD"));

	if (DefaultVertCount != CurrentLevel->Model->Verts.Num())
	{
		//** TEST **//
		// Undo and Redo the placement of the brush.  Keeping track of the number of vertices for both actions.
		GEditor->UndoTransaction();
		int32 VertsFromUndoing = CurrentLevel->Model->Verts.Num();

		GEditor->RedoTransaction();
		int32 VertsFromRedoing = CurrentLevel->Model->Verts.Num();

		//** VERIFY **//
		// Compare the vert count before and after the placement.
		TestEqual<int32>(TEXT("Undo brush placement appears to not have been undone."), DefaultVertCount, VertsFromUndoing);
		TestNotEqual<int32>(TEXT("Redo brush placement appears to not have been redone."), DefaultVertCount, VertsFromRedoing);

		return true;
	}

	UE_LOGF(LogGeometryTests, Error, "Unable to test Undo/Redo since the brush actor was not placed into the level.")
	return false;
}
