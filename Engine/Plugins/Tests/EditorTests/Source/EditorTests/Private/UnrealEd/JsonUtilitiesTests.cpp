// Copyright Epic Games, Inc. All Rights Reserved. 


#include "JsonUtilitiesTests.h"

#include "Modules/ModuleManager.h"

#if WITH_EDITOR
#include "Tests/AutomationEditorCommon.h"
#endif

#include "JsonObjectConverter.h"
#include "JsonSchema/JsonSchemaGenerator.h"
#include "StructUtils/PropertyBag.h"


// Expected values.

// Tests.

#if WITH_DEV_AUTOMATION_TESTS

#define DEBUG_PRINT_JSONS_TO_LOG 0


BEGIN_DEFINE_SPEC(FJsonUtilitiesEditorSpec, "System.Promotion.Editor.JsonUtilities",
	EAutomationTestFlags::EditorContext|EAutomationTestFlags::EngineFilter)
END_DEFINE_SPEC(FJsonUtilitiesEditorSpec)

void TestBasics(
	FAutomationTestBase& Test, const TSharedPtr<FJsonObject>& Schema, const FString& Type, const FString& Description,
	const TOptional<FString>& Title = TOptional<FString>())
{
	Test.TestTrue("Valid schema", Schema.IsValid());
	FString FieldType;
	Test.TestEqual("Type", Schema->TryGetStringField(TEXT("type"), FieldType), !Type.IsEmpty());
	if (!Type.IsEmpty())
	{
		Test.TestEqual("Type", Type, FieldType);
	}
	FString FieldDescription;
	Test.TestEqual("Description", Schema->TryGetStringField(TEXT("description"), FieldDescription), !Description.IsEmpty());
	if (!Description.IsEmpty())
	{
		Test.TestEqual("Description", Description, FieldDescription);
	}
	FString FieldTitle;
	Test.TestEqual("Title", Schema->TryGetStringField(TEXT("title"), FieldTitle), (bool)Title);
	if (Title)
	{
		Test.TestEqual("Title", Title.GetValue(), FieldTitle);
	}
}

void TestFloat(FAutomationTestBase& Test, const TSharedPtr<FJsonObject>& Schema, const FString& Description,
	TOptional<float> Min = TOptional<float>(), TOptional<float> Max = TOptional<float>(), TOptional<float> Default = TOptional<float>())
{
	TestBasics(Test, Schema, "number", Description);
	float FieldMin;
	Test.TestEqual("Min", Schema->TryGetNumberField(TEXT("minimum"), FieldMin), (bool)Min);
	if (Min)
	{		
		Test.TestEqual("Min", Min.GetValue(), FieldMin);
	}
	float FieldMax;
	Test.TestEqual("Max", Schema->TryGetNumberField(TEXT("maximum"), FieldMax), (bool)Max);
	if (Max)
	{		
		Test.TestEqual("Max", Max.GetValue(), FieldMax);
	}
	float FieldDefault;
	Test.TestEqual("Default", Schema->TryGetNumberField(TEXT("default"), FieldDefault), (bool)Default);
	if (Default)
	{
		Test.TestEqual("Default", Default.GetValue(), FieldDefault);
	}
}

void TestInteger(FAutomationTestBase& Test, const TSharedPtr<FJsonObject>& Schema, const FString& Description,
	TOptional<int32> Min = TOptional<int32>(), TOptional<int32> Max = TOptional<int32>(), TOptional<int32> Default = TOptional<int32>())
{
	TestBasics(Test, Schema, "integer", Description);
	int FieldMin;
	Test.TestEqual("Min", Schema->TryGetNumberField(TEXT("minimum"), FieldMin), (bool)Min);
	if (Min)
	{
		Test.TestEqual("Min", Min.GetValue(), FieldMin);
	}
	int FieldMax;
	Test.TestEqual("Max", Schema->TryGetNumberField(TEXT("maximum"), FieldMax), (bool)Max);
	if (Max)
	{
		Test.TestEqual("Max", Max.GetValue(), FieldMax);
	}
	int FieldDefault;
	Test.TestEqual("Default", Schema->TryGetNumberField(TEXT("default"), FieldDefault), (bool)Default);
	if (Default)
	{
		Test.TestEqual("Default", Default.GetValue(), FieldDefault);
	}
}

void TestBoolean(FAutomationTestBase& Test, const TSharedPtr<FJsonObject>& Schema, const FString& Description, TOptional<bool> Default = TOptional<bool>())
{
	TestBasics(Test, Schema, "boolean", Description);
	bool FieldDefault;
	Test.TestEqual("Default", Schema->TryGetBoolField(TEXT("default"), FieldDefault), (bool)Default);
	if (Default)
	{
		Test.TestEqual("Default", Default.GetValue(), FieldDefault);
	}
}

void TestString(FAutomationTestBase& Test, const TSharedPtr<FJsonObject>& Schema, const FString& Description, TOptional<FString> Default = TOptional<FString>())
{
	TestBasics(Test, Schema, "string", Description);
	FString FieldDefault;
	Test.TestEqual("Default", Schema->TryGetStringField(TEXT("default"), FieldDefault), (bool)Default);
	if (Default)
	{
		Test.TestEqual("Default", Default.GetValue(), FieldDefault);
	}
}

void TestObjectRef(FAutomationTestBase& Test, const TSharedPtr<FJsonObject>& Schema, const FString& Description, const FString& Title, bool HasNullDefault)
{
	TestBasics(Test, Schema, "string", Description, Title);
	TSharedPtr<FJsonValue> Default = Schema->TryGetField(TEXT("default"));
	Test.TestEqual("Default", Default.IsValid(), HasNullDefault);
	if (Default.IsValid())
	{				
		Test.TestTrue("Default", Default->IsNull());		
	}
}

void TestClassRef(FAutomationTestBase& Test, const TSharedPtr<FJsonObject>& Schema, const FString& Description, bool HasNullDefault)
{
	TestBasics(Test, Schema, "string", Description, FString("Class"));
	TSharedPtr<FJsonValue> Default = Schema->TryGetField(TEXT("default"));
	Test.TestEqual("Default", Default.IsValid(), HasNullDefault);
	if (Default.IsValid())
	{
		Test.TestTrue("Default", Default->IsNull());
	}
}

void TestEnum(
	FAutomationTestBase& Test, const TSharedPtr<FJsonObject>& Schema, const FString& Description, const FString& Title,
	const TArray<FString>& Enums, TOptional<FString> Default = TOptional<FString>())
{
	TestBasics(Test, Schema, "string", Description, Title);
	TArray<FString> FieldEnums;
	Test.TestEqual("Enums", Schema->TryGetStringArrayField(TEXT("enum"), FieldEnums), !Enums.IsEmpty());
	if (!Enums.IsEmpty())
	{
		Test.TestEqual("Enums", Enums, FieldEnums);
	}
	FString FieldDefault;
	Test.TestEqual("Default", Schema->TryGetStringField(TEXT("default"), FieldDefault), (bool)Default);
	if (Default)
	{
		Test.TestEqual("Default", Default.GetValue(), FieldDefault);
	}
}

void TestRequired(FAutomationTestBase& Test, const TSharedPtr<FJsonObject>& Schema, const TArray<FString>& Required)
{
	TArray<FString> RequiredField;	
	Test.TestEqual("Required", Schema->TryGetStringArrayField(TEXT("required"), RequiredField), !Required.IsEmpty());
	if (!Required.IsEmpty())
	{
		Test.TestEqual("Required", Required.Num(), RequiredField.Num());
		for (auto& Item : Required)
		{
			Test.TestTrue("Required", RequiredField.Contains(Item));
		}
	}
}

template <typename T>
void TestNumericProperties(FAutomationTestBase& Test, const TSharedPtr<FJsonObject>& Schema, const TArray<FString>& PropertyNames, const TArray<FString>& PropertyDescriptions,
	TOptional<T> Min = TOptional<T>(), TOptional<T> Max = TOptional<T>())
{
	const TSharedPtr<FJsonObject>* Properties = nullptr;
	Test.TestTrue("Properties", Schema->TryGetObjectField(TEXT("properties"), Properties));
	if (Properties)
	{		
		Test.TestEqual("Properties", Properties->Get()->Values.Num(), PropertyNames.Num());
		for (int PropIndex = 0; PropIndex < PropertyNames.Num(); ++PropIndex)
		{
			FString PropertyName = PropertyNames[PropIndex];
			FString PropertyDescription = PropIndex < PropertyDescriptions.Num() ? PropertyDescriptions[PropIndex] : "";
			const TSharedPtr<FJsonObject>* Property = nullptr;
			Test.TestTrue("Properties", Properties->Get()->TryGetObjectField(PropertyName, Property));
			if (Property)
			{
				if constexpr (TIsIntegral<T>::Value)
				{
					TestInteger(Test, *Property, PropertyDescription, Min, Max);
				}
				else
				{
					TestFloat(Test, *Property, PropertyDescription, Min, Max);
				}				
			}
		}
	}
}

void TestVector(FAutomationTestBase& Test, const TSharedPtr<FJsonObject>& Schema, const FString& Description, TOptional<FVector> Default = TOptional<FVector>())
{
	TestBasics(Test, Schema, "object", Description, FString("Vector"));
	TArray<FString> PropertyNames({ "x", "y", "z" });
	TestNumericProperties<float>(Test, Schema, PropertyNames, TArray<FString>());
	TestRequired(Test, Schema, PropertyNames);
	
	const TSharedPtr<FJsonObject>* FieldDefault = nullptr;
	Test.TestEqual("Default", Schema->TryGetObjectField(TEXT("default"), FieldDefault), (bool)Default);
	if(Default)
	{
		if (FieldDefault)
		{
			Test.TestEqual("Default", FieldDefault->Get()->Values.Num(), PropertyNames.Num());
			for (int PropIndex = 0; PropIndex < PropertyNames.Num(); ++PropIndex)
			{
				float PropertyValue;
				Test.TestTrue("Default", FieldDefault->Get()->TryGetNumberField(PropertyNames[PropIndex], PropertyValue));
				Test.TestEqual("Default", PropertyValue, (float)Default.GetValue()[PropIndex]);
			}
		}
	}
}

void TestQuat(FAutomationTestBase& Test, const TSharedPtr<FJsonObject>& Schema, const FString& Description, TOptional<FQuat> Default = TOptional<FQuat>())
{
	TestBasics(Test, Schema, "object", Description, FString("Quat"));
	TArray<FString> PropertyNames({ "x", "y", "z", "w"});
	TestNumericProperties<float>(Test, Schema, PropertyNames, TArray<FString>());
	TestRequired(Test, Schema, PropertyNames);

	const TSharedPtr<FJsonObject>* FieldDefault = nullptr;
	Test.TestEqual("Default", Schema->TryGetObjectField(TEXT("default"), FieldDefault), (bool)Default);
	if (Default)
	{
		if (FieldDefault)
		{
			Test.TestEqual("Default", FieldDefault->Get()->Values.Num(), PropertyNames.Num());
			for (auto& PropertyName : PropertyNames)
			{
				double PropertyValue;
				Test.TestTrue("Default", FieldDefault->Get()->TryGetNumberField(PropertyName, PropertyValue));
				if (PropertyName == "x")
				{
					Test.TestEqual("Default", PropertyValue, Default.GetValue().X);
				}
				else if (PropertyName == "y")
				{
					Test.TestEqual("Default", PropertyValue, Default.GetValue().Y);
				}
				else if (PropertyName == "z")
				{
					Test.TestEqual("Default", PropertyValue, Default.GetValue().Z);
				}
				else if (PropertyName == "w")
				{
					Test.TestEqual("Default", PropertyValue, Default.GetValue().W);
				}
			}
		}
	}
}

void TestRotator(FAutomationTestBase& Test, const TSharedPtr<FJsonObject>& Schema, const FString& Description, TOptional<FRotator> Default = TOptional<FRotator>())
{
	TestBasics(Test, Schema, "object", Description, FString("Rotator"));
	TArray<FString> PropertyNames({ "yaw", "pitch", "roll" });
	TArray<FString> PropertyDescriptions(
		{
			"Yaw (degrees) around Z axis",
			"Pitch (degrees) around Y axis",
			"Roll (degrees) around X axis"
		});
	TestNumericProperties<float>(Test, Schema, PropertyNames, PropertyDescriptions);
	TestRequired(Test, Schema, PropertyNames);

	const TSharedPtr<FJsonObject>* FieldDefault = nullptr;
	Test.TestEqual("Default", Schema->TryGetObjectField(TEXT("default"), FieldDefault), (bool)Default);
	if (Default)
	{
		if (FieldDefault)
		{
			Test.TestEqual("Default", FieldDefault->Get()->Values.Num(), PropertyNames.Num());
			for (auto& PropertyName : PropertyNames)
			{
				double PropertyValue;
				Test.TestTrue("Default", FieldDefault->Get()->TryGetNumberField(PropertyName, PropertyValue));				
				if (PropertyName == "yaw")
				{
					Test.TestEqual("Default", PropertyValue, Default.GetValue().Yaw);
				}
				else if (PropertyName == "pitch")
				{
					Test.TestEqual("Default", PropertyValue, Default.GetValue().Pitch);
				}
				else if (PropertyName == "roll")
				{
					Test.TestEqual("Default", PropertyValue, Default.GetValue().Roll);
				}
			}
		}
	}
}

void TestTransform(FAutomationTestBase& Test, const TSharedPtr<FJsonObject>& Schema, const FString& Description, TOptional<FTransform> Default = TOptional<FTransform>())
{
	TestBasics(Test, Schema, "object", Description, FString("Transform"));
	TArray<FString> PropertyNames({ "translation", "rotation", "scale3D" });
	TestRequired(Test, Schema, PropertyNames);

	const TSharedPtr<FJsonObject>* Properties = nullptr;
	Test.TestTrue("Properties", Schema->TryGetObjectField(TEXT("properties"), Properties));
	if (Properties)
	{
		Test.TestEqual("Properties", Properties->Get()->Values.Num(), PropertyNames.Num());
		const TSharedPtr<FJsonObject>* Translation = nullptr;
		Test.TestTrue("Input", Properties->Get()->TryGetObjectField(TEXT("translation"), Translation));
		if (Translation)
		{
			TestVector(Test, *Translation, "Translation of this transformation, as a vector.");
		}

		const TSharedPtr<FJsonObject>* Quat = nullptr;
		Test.TestTrue("Input", Properties->Get()->TryGetObjectField(TEXT("rotation"), Quat));
		if (Quat)
		{
			TestQuat(Test, *Quat, "Rotation of this transformation, as a quaternion.");
		}

		const TSharedPtr<FJsonObject>* Scale = nullptr;
		Test.TestTrue("Input", Properties->Get()->TryGetObjectField(TEXT("scale3D"), Scale));
		if (Scale)
		{
			TestVector(Test, *Scale, "3D scale (always applied in local space) as a vector.");
		}
	}
	
	if (Default)
	{
		const TSharedPtr<FJsonObject>* FieldDefault = nullptr;
		Test.TestTrue("Default", Schema->TryGetObjectField(TEXT("default"), FieldDefault));
		if (FieldDefault)
		{
			Test.TestEqual("Properties", FieldDefault->Get()->Values.Num(), PropertyNames.Num());
			{
				const TSharedPtr<FJsonObject>* Translation = nullptr;
				Test.TestTrue("Default", FieldDefault->Get()->TryGetObjectField(TEXT("translation"), Translation));
				if (Translation)
				{
					UScriptStruct* VectorStruct = TBaseStructure<FVector>::Get();
					FVector ParsedVector;
					FJsonObjectConverter::JsonObjectToUStruct(
						Translation->ToSharedRef(),
						VectorStruct,
						&ParsedVector
					);
					Test.TestEqual("Default", ParsedVector, Default.GetValue().GetLocation());
				}
			}
			
			{
				const TSharedPtr<FJsonObject>* Quat = nullptr;
				Test.TestTrue("Default", FieldDefault->Get()->TryGetObjectField(TEXT("rotation"), Quat));
				if (Quat)
				{
					UScriptStruct* QuatStruct = TBaseStructure<FQuat>::Get();
					FQuat ParsedQuat;
					FJsonObjectConverter::JsonObjectToUStruct(
						Quat->ToSharedRef(),
						QuatStruct,
						&ParsedQuat
					);
					Test.TestEqual("Default", ParsedQuat, Default.GetValue().GetRotation());
				}
			}
			
			{
				const TSharedPtr<FJsonObject>* Scale = nullptr;
				Test.TestTrue("Default", FieldDefault->Get()->TryGetObjectField(TEXT("scale3D"), Scale));
				if (Scale)
				{
					UScriptStruct* VectorStruct = TBaseStructure<FVector>::Get();
					FVector ParsedVector;
					FJsonObjectConverter::JsonObjectToUStruct(
						Scale->ToSharedRef(),
						VectorStruct,
						&ParsedVector
					);
					Test.TestEqual("Default", ParsedVector, Default.GetValue().GetScale3D());
				}
			}			
		}
	}	
}

void TestLinearColor(FAutomationTestBase& Test, const TSharedPtr<FJsonObject>& Schema, const FString& Description, TOptional<FLinearColor> Default = TOptional<FLinearColor>())
{
	TestBasics(Test, Schema, "object", Description, FString("LinearColor"));
	TArray<FString> PropertyNames({ "r", "g", "b", "a" });
	TestNumericProperties<float>(Test, Schema, PropertyNames, TArray<FString>());
	TestRequired(Test, Schema, PropertyNames);

	const TSharedPtr<FJsonObject>* FieldDefault = nullptr;
	Test.TestEqual("Default", Schema->TryGetObjectField(TEXT("default"), FieldDefault), (bool)Default);
	if (Default)
	{
		if (FieldDefault)
		{
			Test.TestEqual("Default", FieldDefault->Get()->Values.Num(), PropertyNames.Num());
			for (auto& PropertyName : PropertyNames)
			{
				float PropertyValue;
				Test.TestTrue("Default", FieldDefault->Get()->TryGetNumberField(PropertyName, PropertyValue));
				if (PropertyName == "r")
				{
					Test.TestEqual("Default", PropertyValue, Default.GetValue().R);
				}
				else if (PropertyName == "g")
				{
					Test.TestEqual("Default", PropertyValue, Default.GetValue().G);
				}
				else if (PropertyName == "b")
				{
					Test.TestEqual("Default", PropertyValue, Default.GetValue().B);
				}
				else if (PropertyName == "a")
				{
					Test.TestEqual("Default", PropertyValue, Default.GetValue().A);
				}
			}
		}
	}
}

void TestColor(FAutomationTestBase& Test, const TSharedPtr<FJsonObject>& Schema, const FString& Description, TOptional<FColor> Default = TOptional<FColor>())
{
	TestBasics(Test, Schema, "object", Description, FString("Color"));
	TArray<FString> PropertyNames({ "r", "g", "b", "a" });
	TestNumericProperties<int32>(Test, Schema, PropertyNames, TArray<FString>(), 0, 255);
	TestRequired(Test, Schema, PropertyNames);

	const TSharedPtr<FJsonObject>* FieldDefault = nullptr;
	Test.TestEqual("Default", Schema->TryGetObjectField(TEXT("default"), FieldDefault), (bool)Default);
	if (Default)
	{
		if (FieldDefault)
		{
			Test.TestEqual("Default", FieldDefault->Get()->Values.Num(), PropertyNames.Num());
			for (auto& PropertyName : PropertyNames)
			{
				uint8 PropertyValue;
				Test.TestTrue("Default", FieldDefault->Get()->TryGetNumberField(PropertyName, PropertyValue));
				if (PropertyName == "r")
				{
					Test.TestEqual("Default", PropertyValue, Default.GetValue().R);
				}
				else if (PropertyName == "g")
				{
					Test.TestEqual("Default", PropertyValue, Default.GetValue().G);
				}
				else if (PropertyName == "b")
				{
					Test.TestEqual("Default", PropertyValue, Default.GetValue().B);
				}
				else if (PropertyName == "a")
				{
					Test.TestEqual("Default", PropertyValue, Default.GetValue().A);
				}
			}
		}
	}
}

bool TestCustomCallback(const FProperty* Property, const FString& ParameterDefaultString, const TSharedRef<FJsonObject>& OutputSchema)
{
	if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{	
		if (StructProperty->Struct == TBaseStructure<FSoftClassPath>::Get())
		{
			if (ParameterDefaultString.IsEmpty())
			{
				OutputSchema->SetStringField(TEXT("type"), TEXT("string"));
			}
			else
			{
				OutputSchema->SetStringField(TEXT("default"), ParameterDefaultString);
			}
			return true;
		}		
	}
	return false;
}

void FJsonUtilitiesEditorSpec::Define()
{	
	BeforeEach([this]() -> void
		{
			// Loading this module will enable the collection of metadata for the schemas.
			static const TCHAR* JsonUtilitiesEditorModuleName = TEXT("JsonUtilitiesEditor");
			const IModuleInterface* Module = FModuleManager::Get().LoadModule(JsonUtilitiesEditorModuleName);
			TestTrue(
				FString::Printf(TEXT("Load module '%s'."), JsonUtilitiesEditorModuleName), 
				LIKELY(Module));
		});
	
	Describe(
		"Should generate JSON schemas from UClasses/UStructs/UProperties/UFunctions.", 
		[this]() -> void
		{
			It(
				"Should generate valid JSON schema from float property.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					FProperty* Prop = Struct->FindPropertyByName("TestFloat");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Prop);
					TestFloat(*this, Schema, "TestFloatDesc", -101.f, 102.f);
				});

			It(
				"Should generate valid JSON schema from int property.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					FProperty* Prop = Struct->FindPropertyByName("TestInt");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Prop);
					TestInteger(*this, Schema, "TestIntDesc", -201, 202);
				});

			It(
				"Should generate valid JSON schema from bool property.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					FProperty* Prop = Struct->FindPropertyByName("TestBool");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Prop);
					TestBoolean(*this, Schema, "TestBoolDesc");
				});

			It(
				"Should generate valid JSON schema from string property.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					FProperty* Prop = Struct->FindPropertyByName("TestString");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Prop);
					TestString(*this, Schema, "TestStringDesc");
				});

			It(
				"Should generate valid JSON schema from name property.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					FProperty* Prop = Struct->FindPropertyByName("TestName");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Prop);
					TestString(*this, Schema, "TestNameDesc");
				});

			It(
				"Should generate valid JSON schema from enum property.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					FProperty* Prop = Struct->FindPropertyByName("TestEnum");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Prop);
					TestEnum(*this, Schema, "TestEnumDesc", "EJsonUtilitiesFakeEnum", TArray<FString>({"Zero", "One"}));
				});

			It(
				"Should generate valid JSON schema from UObject property.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					FProperty* Prop = Struct->FindPropertyByName("TestObject");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Prop);
					TestObjectRef(*this, Schema, "TestObjectDesc", FString(TEXT("Actor")), false);
				});

			It(
				"Should generate valid JSON schema from SoftObject property.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					FProperty* Prop = Struct->FindPropertyByName("TestSoftObject");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Prop);
					TestObjectRef(*this, Schema, "TestSoftObjectDesc", FString(TEXT("Actor")), false);
				});

			It(
				"Should generate valid JSON schema from UClass property.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					FProperty* Prop = Struct->FindPropertyByName("TestClass");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Prop);
					TestClassRef(*this, Schema, "TestClassDesc", false);
				});

			It(
				"Should generate valid JSON schema from SoftClass property.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					FProperty* Prop = Struct->FindPropertyByName("TestSoftClass");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Prop);
					TestClassRef(*this, Schema, "TestSoftClassDesc", false);
				});

			It(
				"Should generate valid JSON schema from Struct property.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					FProperty* Prop = Struct->FindPropertyByName("TestNestedStruct");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Prop);
					TestBasics(*this, Schema, "object", "TestNestedStructDesc", FString("JsonUtilitiesFakeNestedStruct"));
					const TSharedPtr<FJsonObject>* Properties = nullptr;
					TestTrue("Properties", Schema->TryGetObjectField(TEXT("properties"), Properties));
					if (Properties)
					{
						const TSharedPtr<FJsonObject>* Property = nullptr;
						TestTrue("Properties", Properties->Get()->TryGetObjectField(TEXT("TestNestedStructFloat"), Property));
						if (Property)
						{
							TestFloat(*this, *Property, "TestNestedFloatDesc", -301.f, 302.f);
							Property = nullptr;
						}
					}
				});

			It(
				"Should generate valid JSON schema from an Array property.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					FProperty* Prop = Struct->FindPropertyByName("TestArray");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Prop);
					TestBasics(*this, Schema, "array", "TestArrayDesc");
					const TSharedPtr<FJsonObject>* Items = nullptr;
					TestTrue("Items", Schema->TryGetObjectField(TEXT("items"), Items));
					if (Items)
					{
						TestFloat(*this, *Items, "");
					}
				});

			It(
				"Should generate valid JSON schema from a fixed-size array property.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					FProperty* Prop = Struct->FindPropertyByName("TestFixedArray");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Prop);
					TestBasics(*this, Schema, "array", "TestFixedArrayDesc");
					const TSharedPtr<FJsonObject>* Items = nullptr;
					TestTrue("Items", Schema->TryGetObjectField(TEXT("items"), Items));
					if (Items)
					{
						TestFloat(*this, *Items, "");
					}
					int32 MinItems = 0;
					TestTrue("MinItems", Schema->TryGetNumberField(TEXT("minItems"), MinItems));
					TestEqual("MinItems", 3, MinItems);
					int32 MaxItems = 0;
					TestTrue("MaxItems", Schema->TryGetNumberField(TEXT("maxItems"), MaxItems));
					TestEqual("MaxItems", 3, MaxItems);
				});

			It(
				"Should generate valid JSON schema from an EditFixedSize array property with instance memory.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					FProperty* Prop = Struct->FindPropertyByName("TestEditFixedArray");
					FJsonUtilitiesFakeStruct Instance;
					Instance.TestEditFixedArray.Add(1.0f);
					Instance.TestEditFixedArray.Add(2.0f);
					Instance.TestEditFixedArray.Add(3.0f);
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(
						Prop, FJsonSchemaPropertyFilter(), nullptr, &Instance.TestEditFixedArray);
					TestBasics(*this, Schema, "array", "TestEditFixedArrayDesc");
					const TSharedPtr<FJsonObject>* Items = nullptr;
					TestTrue("Items", Schema->TryGetObjectField(TEXT("items"), Items));
					if (Items)
					{
						TestFloat(*this, *Items, "");
					}
					int32 MinItems = 0;
					TestTrue("MinItems", Schema->TryGetNumberField(TEXT("minItems"), MinItems));
					TestEqual("MinItems", 3, MinItems);
					int32 MaxItems = 0;
					TestTrue("MaxItems", Schema->TryGetNumberField(TEXT("maxItems"), MaxItems));
					TestEqual("MaxItems", 3, MaxItems);
				});

			It(
				"Should not emit min/maxItems for an EditFixedSize array property with an empty instance.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					FProperty* Prop = Struct->FindPropertyByName("TestEditFixedArray");
					FJsonUtilitiesFakeStruct Instance;  // TestEditFixedArray is empty
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(
						Prop, FJsonSchemaPropertyFilter(), nullptr, &Instance.TestEditFixedArray);
					TestBasics(*this, Schema, "array", "TestEditFixedArrayDesc");
					int32 Unused = 0;
					TestFalse("No MinItems", Schema->TryGetNumberField(TEXT("minItems"), Unused));
					TestFalse("No MaxItems", Schema->TryGetNumberField(TEXT("maxItems"), Unused));
				});

			It(
				"Should generate valid JSON schema from an EditFixedSize Set property with instance memory.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					FProperty* Prop = Struct->FindPropertyByName("TestEditFixedSet");
					FJsonUtilitiesFakeStruct Instance;
					Instance.TestEditFixedSet.Add(1.0f);
					Instance.TestEditFixedSet.Add(2.0f);
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(
						Prop, FJsonSchemaPropertyFilter(), nullptr, &Instance.TestEditFixedSet);
					TestBasics(*this, Schema, "array", "TestEditFixedSetDesc");
					bool UniqueItems = false;
					TestTrue("UniqueItems", Schema->TryGetBoolField(TEXT("uniqueItems"), UniqueItems));
					TestTrue("UniqueItems", UniqueItems);
					const TSharedPtr<FJsonObject>* Items = nullptr;
					TestTrue("Items", Schema->TryGetObjectField(TEXT("items"), Items));
					if (Items)
					{
						TestFloat(*this, *Items, "");
					}
					int32 MinItems = 0;
					TestTrue("MinItems", Schema->TryGetNumberField(TEXT("minItems"), MinItems));
					TestEqual("MinItems", 2, MinItems);
					int32 MaxItems = 0;
					TestTrue("MaxItems", Schema->TryGetNumberField(TEXT("maxItems"), MaxItems));
					TestEqual("MaxItems", 2, MaxItems);
				});

			It(
				"Should generate valid JSON schema from an EditFixedSize Map property with instance memory.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					FProperty* Prop = Struct->FindPropertyByName("TestEditFixedMap");
					FJsonUtilitiesFakeStruct Instance;
					Instance.TestEditFixedMap.Add(TEXT("KeyA"), 1.0f);
					Instance.TestEditFixedMap.Add(TEXT("KeyB"), 2.0f);
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(
						Prop, FJsonSchemaPropertyFilter(), nullptr, &Instance.TestEditFixedMap);
					TestBasics(*this, Schema, "object", "TestEditFixedMapDesc");
					const TSharedPtr<FJsonObject>* AdditionalProperties = nullptr;
					TestTrue("AdditionalProperties", Schema->TryGetObjectField(TEXT("additionalProperties"), AdditionalProperties));
					if (AdditionalProperties)
					{
						TestFloat(*this, *AdditionalProperties, "");
					}
					int32 MinProperties = 0;
					TestTrue("MinProperties", Schema->TryGetNumberField(TEXT("minProperties"), MinProperties));
					TestEqual("MinProperties", 2, MinProperties);
					int32 MaxProperties = 0;
					TestTrue("MaxProperties", Schema->TryGetNumberField(TEXT("maxProperties"), MaxProperties));
					TestEqual("MaxProperties", 2, MaxProperties);
				});

			It(
				"Should generate valid JSON schema from a Set property.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					FProperty* Prop = Struct->FindPropertyByName("TestSet");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Prop);
					TestBasics(*this, Schema, "array", "TestSetDesc");
					bool UniqueItems = false;
					TestTrue("Items", Schema->TryGetBoolField(TEXT("uniqueItems"), UniqueItems));
					TestTrue("Items", UniqueItems);
					const TSharedPtr<FJsonObject>* Items = nullptr;
					TestTrue("Items", Schema->TryGetObjectField(TEXT("items"), Items));
					if (Items)
					{
						TestFloat(*this, *Items, "");
					}
				});

			It(
				"Should generate valid JSON schema from a Map property w/ String key.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					FProperty* Prop = Struct->FindPropertyByName("TestMap");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Prop);
					TestBasics(*this, Schema, "object", "TestMapDesc");
					const TSharedPtr<FJsonObject>* Items = nullptr;
					TestTrue("Items", Schema->TryGetObjectField(TEXT("additionalProperties"), Items));
					if (Items)
					{
						TestFloat(*this, *Items, "");
					}
				});

			It(
				"Should generate valid JSON schema from a Map property w/ Name key.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					FProperty* Prop = Struct->FindPropertyByName("TestNameMap");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Prop);
					TestBasics(*this, Schema, "object", "TestNameMapDesc");
					const TSharedPtr<FJsonObject>* Items = nullptr;
					TestTrue("Items", Schema->TryGetObjectField(TEXT("additionalProperties"), Items));
					if (Items)
					{
						TestFloat(*this, *Items, "");
					}
				});

			It(
				"Should generate valid JSON schema from structs.",
				[this]() -> void
				{
					UStruct* Struct = FJsonUtilitiesFakeStruct::StaticStruct();
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::UStructToJsonSchemaObject(Struct);
					TestBasics(*this, Schema, "object", "TestStructDesc", FString("JsonUtilitiesFakeStruct"));
					const TSharedPtr<FJsonObject>* Properties = nullptr;
					TestTrue("Properties", Schema->TryGetObjectField(TEXT("properties"), Properties));
					if (Properties)
					{
						{
							const TSharedPtr<FJsonObject>* Property = nullptr;
							TestTrue("Properties", Properties->Get()->TryGetObjectField(TEXT("TestFloat"), Property));
							if (Property)
							{
								TestFloat(*this, *Property, "TestFloatDesc", -101.f, 102.f);
							}
						}

						{
							const TSharedPtr<FJsonObject>* Property = nullptr;
							TestTrue("Properties", Properties->Get()->TryGetObjectField(TEXT("TestInt"), Property));
							if (Property)
							{
								TestInteger(*this, *Property, "TestIntDesc", -201, 202);
							}
						}

						{
							const TSharedPtr<FJsonObject>* Property = nullptr;
							TestTrue("Properties", Properties->Get()->TryGetObjectField(TEXT("TestBool"), Property));
							if (Property)
							{
								TestBoolean(*this, *Property, "TestBoolDesc");
							}
						}

						{
							const TSharedPtr<FJsonObject>* Property = nullptr;
							TestTrue("Properties", Properties->Get()->TryGetObjectField(TEXT("TestString"), Property));
							if (Property)
							{
								TestString(*this, *Property, "TestStringDesc");
							}
						}

						{
							const TSharedPtr<FJsonObject>* Property = nullptr;
							TestTrue("Properties", Properties->Get()->TryGetObjectField(TEXT("TestName"), Property));
							if (Property)
							{
								TestString(*this, *Property, "TestNameDesc");
							}
						}

						{
							const TSharedPtr<FJsonObject>* Property = nullptr;
							TestTrue("Properties", Properties->Get()->TryGetObjectField(TEXT("TestObject"), Property));
							if (Property)
							{
								TestObjectRef(*this, *Property, "TestObjectDesc", FString(TEXT("Actor")), false);
							}
						}

						{
							const TSharedPtr<FJsonObject>* Property = nullptr;
							TestTrue("Properties", Properties->Get()->TryGetObjectField(TEXT("TestSoftObject"), Property));
							if (Property)
							{
								TestObjectRef(*this, *Property, "TestSoftObjectDesc", FString(TEXT("Actor")), false);
							}
						}

						{
							const TSharedPtr<FJsonObject>* Property = nullptr;
							TestTrue("Properties", Properties->Get()->TryGetObjectField(TEXT("TestClass"), Property));
							if (Property)
							{
								TestClassRef(*this, *Property, "TestClassDesc", false);
							}
						}

						{
							const TSharedPtr<FJsonObject>* Property = nullptr;
							TestTrue("Properties", Properties->Get()->TryGetObjectField(TEXT("TestsoftClass"), Property));
							if (Property)
							{
								TestClassRef(*this, *Property, "TestSoftClassDesc", false);
							}
						}

						{
							const TSharedPtr<FJsonObject>* Property = nullptr;
							TestTrue("Properties", Properties->Get()->TryGetObjectField(TEXT("TestNestedStruct"), Property));
							if (Property)
							{
								TestBasics(*this, *Property, "object", "TestNestedStructDesc", FString("JsonUtilitiesFakeNestedStruct"));
							}
						}

						{
							const TSharedPtr<FJsonObject>* Property = nullptr;
							TestTrue("Properties", Properties->Get()->TryGetObjectField(TEXT("TestArray"), Property));
							if (Property)
							{
								TestBasics(*this, *Property, "array", "TestArrayDesc");
							}
						}

						{
							const TSharedPtr<FJsonObject>* Property = nullptr;
							TestTrue("Properties", Properties->Get()->TryGetObjectField(TEXT("TestSet"), Property));
							if (Property)
							{
								TestBasics(*this, *Property, "array", "TestSetDesc");
							}
						}

						{
							const TSharedPtr<FJsonObject>* Property = nullptr;
							TestTrue("Properties", Properties->Get()->TryGetObjectField(TEXT("TestMap"), Property));
							if (Property)
							{
								TestBasics(*this, *Property, "object", "TestMapDesc");
							}
						}

						{
							const TSharedPtr<FJsonObject>* Property = nullptr;
							TestTrue("Properties", Properties->Get()->TryGetObjectField(TEXT("TestNameMap"), Property));
							if (Property)
							{
								TestBasics(*this, *Property, "object", "TestNameMapDesc");
							}
						}
					}
				});

			It(
				"Should generate valid JSON schema from a UClass.",
				[this]() -> void
				{
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::UStructToJsonSchemaObject<UJsonUtilitiesFakeClass>();
					TestBasics(*this, Schema, "object", "TestClassDesc", FString("JsonUtilitiesFakeClass"));
					const TSharedPtr<FJsonObject>* Properties = nullptr;
					TestTrue("Properties", Schema->TryGetObjectField(TEXT("properties"), Properties));
					if (Properties)
					{
						const TSharedPtr<FJsonObject>* Property = nullptr;
						TestTrue("Properties", Properties->Get()->TryGetObjectField(TEXT("TestFloat"), Property));
						if (Property)
						{
							TestFloat(*this, *Property, "TestFloatDesc", -101.f, 102.f);
						}
					}
				});

			It(
				"Should generate JSON schema from a basic UFunction.",
				[this]() -> void
				{
					UFunction* FunctionPtr = UJsonUtilitiesFakeClass::StaticClass()->FindFunctionByName("TestFunc");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::UStructToJsonSchemaObject(FunctionPtr);
					TestTrue("Schema", Schema.IsValid());
					TestBasics(*this, Schema, "", "TestFuncDesc");
					const TSharedPtr<FJsonObject>* InputSchema = nullptr;
					TestTrue("Input", Schema->TryGetObjectField(TEXT("inputSchema"), InputSchema));
					if (InputSchema)
					{
						TestBasics(*this, *InputSchema, "object", "");

						const TSharedPtr<FJsonObject>* Properties = nullptr;
						TestTrue("Input", InputSchema->Get()->TryGetObjectField(TEXT("properties"), Properties));
						if (Properties)
						{
							TestEqual("Input", Properties->Get()->Values.Num(), 13);
							TestRequired(*this, *InputSchema, TArray<FString>({ "requiredParam" }));

							{
								const TSharedPtr<FJsonObject>* Property = nullptr;
								TestTrue("Input", Properties->Get()->TryGetObjectField(TEXT("requiredParam"), Property));
								if (Property)
								{
									TestInteger(*this, *Property, "TestRequiredDesc");
								}
							}

							{
								const TSharedPtr<FJsonObject>* Property = nullptr;
								TestTrue("Input", Properties->Get()->TryGetObjectField(TEXT("optionalParam"), Property));
								if (Property)
								{
									TestFloat(*this, *Property, "TestOptionalDesc");
								}
							}

							{
								const TSharedPtr<FJsonObject>* Property = nullptr;
								TestTrue("Input", Properties->Get()->TryGetObjectField(TEXT("boolParam"), Property));
								if (Property)
								{
									TestBoolean(*this, *Property, "TestBoolDesc", true);
								}
							}

							{
								const TSharedPtr<FJsonObject>* Property = nullptr;
								TestTrue("Input", Properties->Get()->TryGetObjectField(TEXT("floatParam"), Property));
								if (Property)
								{
									TestFloat(*this, *Property, "TestFloatDesc", TOptional<float>(), TOptional<float>(), 1.23f);
								}
							}

							{
								const TSharedPtr<FJsonObject>* Property = nullptr;
								TestTrue("Input", Properties->Get()->TryGetObjectField(TEXT("enumParam"), Property));
								if (Property)
								{
									TestEnum(
										*this, *Property, "TestEnumDesc", "EJsonUtilitiesFakeEnum",
										TArray<FString>({"Zero", "One"}), FString("One"));
								}
							}

							{
								const TSharedPtr<FJsonObject>* Property = nullptr;
								TestTrue("Input", Properties->Get()->TryGetObjectField(TEXT("stringParam"), Property));
								if (Property)
								{
									TestString(*this, *Property, "TestStringDesc", FString("foo"));
								}
							}

							{
								const TSharedPtr<FJsonObject>* Property = nullptr;
								TestTrue("Input", Properties->Get()->TryGetObjectField(TEXT("nameParam"), Property));
								if (Property)
								{
									TestString(*this, *Property, "TestNameDesc", FString("bar"));
								}
							}

							{
								const TSharedPtr<FJsonObject>* Property = nullptr;
								TestTrue("Input", Properties->Get()->TryGetObjectField(TEXT("objectParam"), Property));
								if (Property)
								{
									TestObjectRef(*this, *Property, "TestObjectDesc", FString(TEXT("Object")), true);
								}
							}

							{
								const TSharedPtr<FJsonObject>* Property = nullptr;
								TestTrue("Input", Properties->Get()->TryGetObjectField(TEXT("linearColorParam"), Property));
								if (Property)
								{
									TestLinearColor(*this, *Property, "TestLinearColorDesc", FLinearColor(0.1f, 0.2f, 0.3f, 0.4f));
								}
							}

							{
								const TSharedPtr<FJsonObject>* Property = nullptr;
								TestTrue("Input", Properties->Get()->TryGetObjectField(TEXT("colorParam"), Property));
								if (Property)
								{
									TestColor(*this, *Property, "TestColorDesc", FColor(10, 20, 30, 40));
								}
							}

							{
								const TSharedPtr<FJsonObject>* Property = nullptr;
								TestTrue("Input", Properties->Get()->TryGetObjectField(TEXT("rotatorParam"), Property));
								if (Property)
								{
									TestRotator(*this, *Property, "TestRotatorDesc", FRotator(20, 10, 30));
								}
							}

							{
								const TSharedPtr<FJsonObject>* Property = nullptr;
								TestTrue("Input", Properties->Get()->TryGetObjectField(TEXT("vectorParam"), Property));
								if (Property)
								{
									TestVector(*this, *Property, "TestVectorDesc", FVector(1.f, 2.f, 3.f));
								}
							}
						}
					}

					const TSharedPtr<FJsonObject>* OutputSchema = nullptr;
					TestTrue("Output", Schema->TryGetObjectField(TEXT("outputSchema"), OutputSchema));
					if (OutputSchema)
					{
						TestEqual("Output", OutputSchema->Get()->Values.Num(), 3);
						TestBasics(*this, *OutputSchema, "object", "");
						TestRequired(*this, *OutputSchema, TArray<FString>({ "returnValue" }));

						const TSharedPtr<FJsonObject>* Properties = nullptr;
						TestTrue("Output", OutputSchema->Get()->TryGetObjectField(TEXT("properties"), Properties));
						if (Properties)
						{
							TestEqual("Output", Properties->Get()->Values.Num(), 1);
							const TSharedPtr<FJsonObject>* ReturnValue = nullptr;
							TestTrue("Output", Properties->Get()->TryGetObjectField(TEXT("returnValue"), ReturnValue));
							if (ReturnValue)
							{
								TestFloat(*this, *ReturnValue, "TestReturnDesc");
							}
						}
					}
				});

			It(
				"Should generate JSON schema from a UFunction with custom structs with defaults.",
				[this]() -> void
				{
					UFunction* FunctionPtr = UJsonUtilitiesFakeClass::StaticClass()->FindFunctionByName("TestEngineStructFunc");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::UStructToJsonSchemaObject(FunctionPtr);
					TestTrue("Schema", Schema.IsValid());
					TestBasics(*this, Schema, "", "TestEngineStructFuncDesc");

					const TSharedPtr<FJsonObject>* InputSchema = nullptr;
					TestTrue("Input", Schema->TryGetObjectField(TEXT("inputSchema"), InputSchema));
					if (InputSchema)
					{
						TestBasics(*this, *InputSchema, "object", "");
						TestRequired(*this, *InputSchema, TArray<FString>());
						const TSharedPtr<FJsonObject>* Properties = nullptr;
						TestTrue("Input", InputSchema->Get()->TryGetObjectField(TEXT("properties"), Properties));
						if (Properties)
						{
							const TSharedPtr<FJsonObject>* Transform = nullptr;
							TestTrue("Input", Properties->Get()->TryGetObjectField(TEXT("transform"), Transform));
							if (Transform)
							{
								TestTransform(*this, *Transform, "TestEngineStructFuncParamTransformDesc",
									FTransform(FQuat(0.1, 0.2, 0.3, 0.4), FVector(1, 2, 3), FVector(10, 20, 30)));
							}
						}
					}

					const TSharedPtr<FJsonObject>* OutputSchema = nullptr;
					TestTrue("Output", Schema->TryGetObjectField(TEXT("outputSchema"), OutputSchema));
					if (OutputSchema)
					{
						TestEqual("Output", OutputSchema->Get()->Values.Num(), 3);
						TestBasics(*this, *OutputSchema, "object", "");
						TestRequired(*this, *OutputSchema, TArray<FString>({ "returnValue" }));
						const TSharedPtr<FJsonObject>* Properties = nullptr;
						TestTrue("Output", OutputSchema->Get()->TryGetObjectField(TEXT("properties"), Properties));
						if (Properties)
						{
							TestEqual("Output", Properties->Get()->Values.Num(), 1);
							const TSharedPtr<FJsonObject>* ReturnValue = nullptr;
							TestTrue("Output", Properties->Get()->TryGetObjectField(TEXT("returnValue"), ReturnValue));
							if (ReturnValue)
							{
								TestEqual("Output", ReturnValue->Get()->Values.Num(), 5);
								TestBasics(
									*this, *ReturnValue, "object", "TestEngineStructFuncReturnValueDesc",
									FString("JsonUtilitiesFakeNestedStruct"));
								TestRequired(*this, *ReturnValue, TArray<FString>({ "testNestedStructFloat" }));
								TestTrue("Output", ReturnValue->Get()->TryGetObjectField(TEXT("properties"), Properties));
								if (Properties)
								{
									TestEqual("Output", Properties->Get()->Values.Num(), 1);
									const TSharedPtr<FJsonObject>* FloatProperty = nullptr;
									TestTrue("Output", Properties->Get()->TryGetObjectField(TEXT("testNestedStructFloat"), FloatProperty));
									if (FloatProperty)
									{
										TestFloat(*this, *FloatProperty, "TestNestedFloatDesc", -301.f, 302.f);
									}
								}
							}
						}
					}
				});

			It(
				"Should not generate JSON schema from an UFunction with non-Json-able params.",
				[this]() -> void
				{									
					UFunction* FunctionPtr = UJsonUtilitiesFakeClass::StaticClass()->FindFunctionByName("TestIllegalFunc");
					if (!TestTrue("Found named function.", LIKELY(FunctionPtr)))
					{
						return;
					}
				
					AddExpectedError(TEXT(".*unhandled during Json schema generation.*"),
						EAutomationExpectedErrorFlags::Contains, INDEX_NONE);
						
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::UStructToJsonSchemaObject(FunctionPtr);
					TestFalse("Schema is invalid.", Schema.IsValid());					
			});

			It(
				"Should generate a fallback FInstancedStruct schema with _structType without instance memory.",
				[this]() -> void
				{
					UStruct* WrapperStruct = FJsonUtilitiesInstancedStructWrapper::StaticStruct();
					FProperty* Prop = WrapperStruct->FindPropertyByName("InstancedStructField");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Prop);
					TestBasics(*this, Schema, "object", "Polymorphic struct. Provide _structType with the UScriptStruct path (e.g. /Script/Module.StructName) to create from scratch.", FString("FInstancedStruct"));
					const TSharedPtr<FJsonObject>* Properties = nullptr;
					TestTrue("Has properties", Schema->TryGetObjectField(TEXT("properties"), Properties));
					if (Properties)
					{
						TestTrue("Has _structType", Properties->Get()->HasField(TEXT("_structType")));
					}
				});

			It(
				"Should generate full FInstancedStruct schema when instance memory is provided.",
				[this]() -> void
				{
					UStruct* WrapperStruct = FJsonUtilitiesInstancedStructWrapper::StaticStruct();
					FProperty* Prop = WrapperStruct->FindPropertyByName("InstancedStructField");
					FInstancedStruct InstancedStruct;
					InstancedStruct.InitializeAs<FJsonUtilitiesInstancedInnerStruct>();
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(
						Prop, FJsonSchemaPropertyFilter(), nullptr, &InstancedStruct);
					TestBasics(*this, Schema, "object", "", FString("JsonUtilitiesInstancedInnerStruct"));
					const TSharedPtr<FJsonObject>* Properties = nullptr;
					TestTrue("Properties", Schema->TryGetObjectField(TEXT("properties"), Properties));
					if (Properties)
					{
						const TSharedPtr<FJsonObject>* FloatField = nullptr;
						TestTrue("FloatValue", Properties->Get()->TryGetObjectField(TEXT("FloatValue"), FloatField));
						if (FloatField)
						{
							TestFloat(*this, *FloatField, "");
						}
						const TSharedPtr<FJsonObject>* StringField = nullptr;
						TestTrue("StringValue", Properties->Get()->TryGetObjectField(TEXT("StringValue"), StringField));
						if (StringField)
						{
							TestString(*this, *StringField, "");
						}
					}
					TestRequired(*this, Schema, TArray<FString>({"FloatValue", "StringValue"}));
				});

			It(
				"Should generate a FInstancedPropertyBag fallback schema without instance memory.",
				[this]() -> void
				{
					UStruct* WrapperStruct = FJsonUtilitiesPropertyBagWrapper::StaticStruct();
					FProperty* Prop = WrapperStruct->FindPropertyByName("PropertyBagField");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Prop);
					TestBasics(*this, Schema, "object", "", FString("FInstancedPropertyBag"));
					TestFalse("No properties field", Schema->HasField(TEXT("properties")));
				});

			It(
				"Should generate full FInstancedPropertyBag schema when instance memory is provided.",
				[this]() -> void
				{
					UStruct* WrapperStruct = FJsonUtilitiesPropertyBagWrapper::StaticStruct();
					FProperty* Prop = WrapperStruct->FindPropertyByName("PropertyBagField");
					FInstancedPropertyBag Bag;
					Bag.AddProperty(FName("TestFloat"), EPropertyBagPropertyType::Float);
					Bag.AddProperty(FName("TestString"), EPropertyBagPropertyType::String);
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(
						Prop, FJsonSchemaPropertyFilter(), nullptr, &Bag);
					TestTrue("Schema", Schema.IsValid());
					if (!Schema.IsValid())
					{
						return;
					}
					FString Type;
					TestTrue("Type field present", Schema->TryGetStringField(TEXT("type"), Type));
					TestEqual("Type", Type, FString("object"));
					const TSharedPtr<FJsonObject>* Properties = nullptr;
					TestTrue("Properties", Schema->TryGetObjectField(TEXT("properties"), Properties));
					if (Properties)
					{
						const TSharedPtr<FJsonObject>* FloatProp = nullptr;
						TestTrue("TestFloat", Properties->Get()->TryGetObjectField(TEXT("TestFloat"), FloatProp));
						if (FloatProp)
						{
							TestFloat(*this, *FloatProp, "");
						}
						const TSharedPtr<FJsonObject>* StringProp = nullptr;
						TestTrue("TestString", Properties->Get()->TryGetObjectField(TEXT("TestString"), StringProp));
						if (StringProp)
						{
							TestString(*this, *StringProp, "");
						}
					}
				});

			It(
				"Should generate a fallback TOptional<FInstancedStruct> schema with _structType without instance memory.",
				[this]() -> void
				{
					UStruct* WrapperStruct = FJsonUtilitiesInstancedStructWrapper::StaticStruct();
					FProperty* Prop = WrapperStruct->FindPropertyByName("OptionalInstancedStructField");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Prop);
					TestBasics(*this, Schema, "object", "Polymorphic struct. Provide _structType with the UScriptStruct path (e.g. /Script/Module.StructName) to create from scratch.", FString("FInstancedStruct"));
					const TSharedPtr<FJsonObject>* Properties = nullptr;
					TestTrue("Has properties", Schema->TryGetObjectField(TEXT("properties"), Properties));
					if (Properties)
					{
						TestTrue("Has _structType", Properties->Get()->HasField(TEXT("_structType")));
					}
				});

			It(
				"Should generate full TOptional<FInstancedStruct> schema when instance memory is provided and the optional is set.",
				[this]() -> void
				{
					UStruct* WrapperStruct = FJsonUtilitiesInstancedStructWrapper::StaticStruct();
					FProperty* Prop = WrapperStruct->FindPropertyByName("OptionalInstancedStructField");
					TOptional<FInstancedStruct> OptionalField;
					OptionalField.Emplace();
					OptionalField->InitializeAs<FJsonUtilitiesInstancedInnerStruct>();
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(
						Prop, FJsonSchemaPropertyFilter(), nullptr, &OptionalField);
					TestBasics(*this, Schema, "object", "", FString("JsonUtilitiesInstancedInnerStruct"));
					const TSharedPtr<FJsonObject>* Properties = nullptr;
					TestTrue("Properties", Schema->TryGetObjectField(TEXT("properties"), Properties));
					if (Properties)
					{
						const TSharedPtr<FJsonObject>* FloatField = nullptr;
						TestTrue("FloatValue", Properties->Get()->TryGetObjectField(TEXT("FloatValue"), FloatField));
						if (FloatField)
						{
							TestFloat(*this, *FloatField, "");
						}
						const TSharedPtr<FJsonObject>* StringField = nullptr;
						TestTrue("StringValue", Properties->Get()->TryGetObjectField(TEXT("StringValue"), StringField));
						if (StringField)
						{
							TestString(*this, *StringField, "");
						}
					}
					TestRequired(*this, Schema, TArray<FString>({"FloatValue", "StringValue"}));
				});

			It(
				"Should generate a fallback TOptional<FInstancedStruct> schema with _structType when instance memory is provided but the optional is not set.",
				[this]() -> void
				{
					UStruct* WrapperStruct = FJsonUtilitiesInstancedStructWrapper::StaticStruct();
					FProperty* Prop = WrapperStruct->FindPropertyByName("OptionalInstancedStructField");
					TOptional<FInstancedStruct> OptionalField; // not set
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(
						Prop, FJsonSchemaPropertyFilter(), nullptr, &OptionalField);
					TestBasics(*this, Schema, "object", "Polymorphic struct. Provide _structType with the UScriptStruct path (e.g. /Script/Module.StructName) to create from scratch.", FString("FInstancedStruct"));
					const TSharedPtr<FJsonObject>* Properties = nullptr;
					TestTrue("Has properties", Schema->TryGetObjectField(TEXT("properties"), Properties));
					if (Properties)
					{
						TestTrue("Has _structType", Properties->Get()->HasField(TEXT("_structType")));
					}
				});

			It(
				"Should generate a fallback TOptional<FInstancedPropertyBag> schema without instance memory.",
				[this]() -> void
				{
					UStruct* WrapperStruct = FJsonUtilitiesPropertyBagWrapper::StaticStruct();
					FProperty* Prop = WrapperStruct->FindPropertyByName("OptionalPropertyBagField");
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(Prop);
					TestBasics(*this, Schema, "object", "", FString("FInstancedPropertyBag"));
					TestFalse("No properties field", Schema->HasField(TEXT("properties")));
				});

			It(
				"Should generate full TOptional<FInstancedPropertyBag> schema when instance memory is provided and the optional is set.",
				[this]() -> void
				{
					UStruct* WrapperStruct = FJsonUtilitiesPropertyBagWrapper::StaticStruct();
					FProperty* Prop = WrapperStruct->FindPropertyByName("OptionalPropertyBagField");
					TOptional<FInstancedPropertyBag> OptionalField;
					OptionalField.Emplace();
					OptionalField->AddProperty(FName("TestFloat"), EPropertyBagPropertyType::Float);
					OptionalField->AddProperty(FName("TestString"), EPropertyBagPropertyType::String);
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(
						Prop, FJsonSchemaPropertyFilter(), nullptr, &OptionalField);
					TestTrue("Schema", Schema.IsValid());
					if (!Schema.IsValid())
					{
						return;
					}
					FString Type;
					TestTrue("Type field present", Schema->TryGetStringField(TEXT("type"), Type));
					TestEqual("Type", Type, FString("object"));
					const TSharedPtr<FJsonObject>* Properties = nullptr;
					TestTrue("Properties", Schema->TryGetObjectField(TEXT("properties"), Properties));
					if (Properties)
					{
						const TSharedPtr<FJsonObject>* FloatProp = nullptr;
						TestTrue("TestFloat", Properties->Get()->TryGetObjectField(TEXT("TestFloat"), FloatProp));
						if (FloatProp)
						{
							TestFloat(*this, *FloatProp, "");
						}
						const TSharedPtr<FJsonObject>* StringProp = nullptr;
						TestTrue("TestString", Properties->Get()->TryGetObjectField(TEXT("TestString"), StringProp));
						if (StringProp)
						{
							TestString(*this, *StringProp, "");
						}
					}
				});

			It(
				"Should generate a fallback TOptional<FInstancedPropertyBag> schema when instance memory is provided but the optional is not set.",
				[this]() -> void
				{
					UStruct* WrapperStruct = FJsonUtilitiesPropertyBagWrapper::StaticStruct();
					FProperty* Prop = WrapperStruct->FindPropertyByName("OptionalPropertyBagField");
					TOptional<FInstancedPropertyBag> OptionalField; // not set
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::FPropertyToJsonSchemaObject(
						Prop, FJsonSchemaPropertyFilter(), nullptr, &OptionalField);
					TestBasics(*this, Schema, "object", "", FString("FInstancedPropertyBag"));
					TestFalse("No properties field", Schema->HasField(TEXT("properties")));
				});

			It(
				"Should support custom callbacks.",
				[this]() -> void
				{
					UFunction* FunctionPtr = UJsonUtilitiesFakeClass::StaticClass()->FindFunctionByName("TestEngineCustomizedFunc");
					FJsonSchemaPropertyFilter::CustomCallback CustomCallback;
					CustomCallback.BindStatic(TestCustomCallback);
					FJsonSchemaPropertyFilter PropertyFilter;
					PropertyFilter.CustomCb = &CustomCallback;
					const TSharedPtr<FJsonObject> Schema = FJsonSchemaGenerator::UStructToJsonSchemaObject(FunctionPtr, PropertyFilter);
					TestTrue("Schema", Schema.IsValid());
					TestBasics(*this, Schema, "", "TestCustomizedFuncDesc");
					const TSharedPtr<FJsonObject>* InputSchema = nullptr;
					TestTrue("Input", Schema->TryGetObjectField(TEXT("inputSchema"), InputSchema));
					if (InputSchema)
					{
						TestBasics(*this, *InputSchema, "object", "");
						TestRequired(*this, *InputSchema, TArray<FString>());
						const TSharedPtr<FJsonObject>* Properties = nullptr;
						TestTrue("Input", InputSchema->Get()->TryGetObjectField(TEXT("properties"), Properties));
						if (Properties)
						{
							const TSharedPtr<FJsonObject>* SoftPath = nullptr;
							TestTrue("Input", Properties->Get()->TryGetObjectField(TEXT("softPath"), SoftPath));
							if (SoftPath)
							{
								TestString(*this, *SoftPath, "TestSoftPathDesc", FString("/Engine/StaticMesh.StaticMesh"));
							}
						}
					}
				});
		});
}

#endif
