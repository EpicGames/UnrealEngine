// Copyright Epic Games, Inc. All Rights Reserved. 


#pragma once


#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "Math/Color.h"
#include "Math/MathFwd.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/PropertyBag.h"

#include "JsonUtilitiesTests.generated.h"


// NOTE_JSON_UTILITIES_EDITOR_TEST_BLUEPRINT_DEFAULTS
// Default values for function parameters in the metadata will only be available if a class is
// BlueprintType and a function is a Blueprint function.

UENUM()
enum class EJsonUtilitiesFakeEnum : uint8
{
	Zero = 0,
	One = 1
};

/**
 * TestNestedStructADesc
 */
USTRUCT(BlueprintType) 
struct FJsonUtilitiesFakeNestedStruct
{
	GENERATED_BODY()
	
	/**
	 * TestNestedFloatDesc
	 */ 
	UPROPERTY(EditAnywhere, Category="Nested", meta=(ClampMin="-301", ClampMax="302"))
	float TestNestedStructFloat = 3.0f;
};


/**
 * TestStructDesc
 */
USTRUCT(BlueprintType)
struct FJsonUtilitiesFakeStruct
{
	GENERATED_BODY()
	
	/**
	 * TestFloatDesc
	 */ 
	UPROPERTY(EditAnywhere, Category="Main", meta=(ClampMin="-101.0", ClampMax="102.0"))
	float TestFloat = 1.0f;
	
	/**
	 * TestIntDesc
	 */ 
	UPROPERTY(EditAnywhere, Category="Main", meta=(ClampMin="-201", ClampMax="202"))
	int32 TestInt = 2;

	/**
	 * TestBoolDesc
	 */ 
	UPROPERTY(EditAnywhere, Category="Main")
	bool TestBool = true;

	/**
	 * TestStringDesc
	 */ 
	UPROPERTY(EditAnywhere, Category="Main")
	FString TestString = "DefaultString";
	
	/**
	 * TestNameDesc
	 */
	UPROPERTY(EditAnywhere, Category = "Main")
	FName TestName = "DefaultName";

	/**
	 * TestEnumDesc
	 */ 
	UPROPERTY(EditAnywhere, Category="Main")
	EJsonUtilitiesFakeEnum TestEnum = EJsonUtilitiesFakeEnum::One;

	/**
	 * TestObjectDesc
	 */
	UPROPERTY(EditAnywhere, Category = "Main")
	TObjectPtr<AActor> TestObject;

	/**
	 * TestSoftObjectDesc
	 */
	UPROPERTY(EditAnywhere, Category = "Main")
	TSoftObjectPtr<AActor> TestSoftObject;

	/**
	 * TestClassDesc
	 */
	UPROPERTY(EditAnywhere, Category = "Main")
	TObjectPtr<UClass> TestClass;

	/**
	 * TestSoftClassDesc
	 */
	UPROPERTY(EditAnywhere, Category = "Main")
	TSoftClassPtr<UClass> TestSoftClass;
	
	/**
	 * TestNestedStructDesc
	 */
	UPROPERTY(EditAnywhere, Category="Main")
	FJsonUtilitiesFakeNestedStruct TestNestedStruct;

	/**
	 * TestArrayDesc
	 */
	UPROPERTY(EditAnywhere, Category = "Main")
	TArray<float> TestArray;

	/**
	 * TestSetDesc
	 */
	UPROPERTY(EditAnywhere, Category = "Main")
	TSet<float> TestSet;

	/**
	 * TestMapDesc
	 */
	UPROPERTY(EditAnywhere, Category = "Main")
	TMap<FString, float> TestMap;

	/**
	 * TestNameMapDesc
	 */
	UPROPERTY(EditAnywhere, Category = "Main")
	TMap<FName, float> TestNameMap;

	/**
	 * TestFixedArrayDesc
	 */
	UPROPERTY(EditAnywhere, Category = "Main")
	float TestFixedArray[3] = {};

	/**
	 * TestEditFixedArrayDesc
	 */
	UPROPERTY(EditAnywhere, EditFixedSize, Category = "Main")
	TArray<float> TestEditFixedArray;

	/**
	 * TestEditFixedSetDesc
	 */
	UPROPERTY(EditAnywhere, EditFixedSize, Category = "Main")
	TSet<float> TestEditFixedSet;

	/**
	 * TestEditFixedMapDesc
	 */
	UPROPERTY(EditAnywhere, EditFixedSize, Category = "Main")
	TMap<FString, float> TestEditFixedMap;
};

/**
 * A simple inner struct used as the concrete type inside FInstancedStruct tests.
 */
USTRUCT()
struct FJsonUtilitiesInstancedInnerStruct
{
	GENERATED_BODY()

	UPROPERTY()
	float FloatValue = 42.0f;

	UPROPERTY()
	FString StringValue = TEXT("hello");
};

/**
 * Wrapper struct that holds an FInstancedStruct for converter and schema tests.
 */
USTRUCT()
struct FJsonUtilitiesInstancedStructWrapper
{
	GENERATED_BODY()

	UPROPERTY()
	FInstancedStruct InstancedStructField;

	UPROPERTY()
	TOptional<FInstancedStruct> OptionalInstancedStructField;
};

/**
 * Wrapper struct that holds an FInstancedPropertyBag for converter and schema tests.
 */
USTRUCT()
struct FJsonUtilitiesPropertyBagWrapper
{
	GENERATED_BODY()

	UPROPERTY()
	FInstancedPropertyBag PropertyBagField;

	UPROPERTY()
	TOptional<FInstancedPropertyBag> OptionalPropertyBagField;
};

/**
 * TestClassDesc
 */
UCLASS(BlueprintType)
class UJsonUtilitiesFakeClass : public UObject
{
	GENERATED_BODY()

	/**
	 * TestFloatDesc
	 */
	UPROPERTY(EditAnywhere, Category = "Main", meta = (ClampMin = "-101.0", ClampMax = "102.0"))
	float TestFloat = 1.0f;
	
	/**
	 * TestFuncDesc	 
	 * @param RequiredParam TestRequiredDesc
	 * @param OptionalParam TestOptionalDesc
	 * @param BoolParam TestBoolDesc
	 * @param FloatParam TestFloatDesc
	 * @param IntParam TestIntDesc
	 * @param EnumParam TestEnumDesc
	 * @param StringParam TestStringDesc
	 * @param NameParam TestNameDesc
	 * @param ObjectParam TestObjectDesc	 
	 * @param LinearColorParam TestLinearColorDesc
	 * @param ColorParam TestColorDesc
	 * @param RotatorParam TestRotatorDesc
	 * @param VectorParam TestVectorDesc
	 * @return TestReturnDesc
	 */
	UFUNCTION(BlueprintCallable, Category="Main") 
	float TestFunc(int RequiredParam, TOptional<float> OptionalParam, bool BoolParam = true, float FloatParam = 1.23f,
		int32 IntParam = 123, EJsonUtilitiesFakeEnum EnumParam = EJsonUtilitiesFakeEnum::One,
		const FString& StringParam = "foo", const FName& NameParam = FName("bar"), const UObject* ObjectParam = nullptr,
		FLinearColor LinearColorParam = FLinearColor(0.1f, 0.2f, 0.3f, 0.4f), FColor ColorParam = FColor(10, 20, 30, 40),
		FRotator RotatorParam = FRotator(20, 10, 30), FVector VectorParam = FVector(1.f, 2.f, 3.f))
	{
		// This function is for testing schema generation for a function with 
		// simpler parameters.
		return 0.0f;
	}
	
	/**
	 * TestEngineStructFuncDesc	  
	 * @param Transform TestEngineStructFuncParamTransformDesc 	 
	 * @return TestEngineStructFuncReturnValueDesc
	 */
	UFUNCTION(BlueprintCallable, Category="Main", meta=(		
		CPP_Default_Transform="(Rotation=(X=0.1,Y=0.2,Z=0.3,W=0.4),Translation=(X=1,Y=2,Z=3),Scale3D=(X=10,Y=20,Z=30))"))
	FJsonUtilitiesFakeNestedStruct TestEngineStructFunc(FTransform Transform)
	{
		// This function is for testing schema generation for a function with
		// more complex struct parameters, with defaults.
		// UHT does allow some structs of be initialized in the declaration, like FVector.
		// UHT does not allow most other structs to be initialized in the declaration, like 
		// FTransform or arbitrary structs and for those we  need to use CPP_Default_XXX. 
		// The real purpose of this function is for testing how these defaults end up looking in
		// the generated JSON schema.

		return FJsonUtilitiesFakeNestedStruct();
	}

	/**
	 * TestCustomizedFuncDesc
	 * @param SoftPath TestSoftPathDesc
	 */
	UFUNCTION(BlueprintCallable, Category = "Main", meta = (CPP_Default_SoftPath = "/Engine/StaticMesh.StaticMesh"))
	void TestEngineCustomizedFunc(FSoftClassPath SoftPath)
	{
	}
	
	/**
	 * TestIllegalFuncDesc
	 * @return TestIllegalFuncReturnValueDesc
	 */
	UFUNCTION(BlueprintCallable, Category="Main")
	UObject* TestIllegalFunc(TMap<FVector, float>& Arg)
	{
		// This function is for testing schema generation for a function with some parameters that 
		// can't be represented in the JSON schema.

		return nullptr;
	}
};

