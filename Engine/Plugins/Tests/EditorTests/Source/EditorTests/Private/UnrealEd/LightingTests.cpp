// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"
#include "EngineGlobals.h"
#include "Components/PointLightComponent.h"
#include "Engine/PointLight.h"
#include "Editor.h"
#include "EngineUtils.h"

// Automation
#include "Tests/AutomationEditorCommon.h"

// Misc
#include "ScopedTransaction.h"
#include "Engine/Selection.h"
#include "EditorModeManager.h"

#define POINTLIGHT_TRANSFORM			FTransform()
#define POINT_LIGHT_UPDATED_LOCATION	FVector(50.f, 50.f, 50.f)
#define POINT_LIGHT_UPDATED_ROTATION	FRotator(-60.0f, -110.0f, -91.0f)		// X: -91, Y: -60, Z: -110
#define POINT_LIGHT_UPDATED_SCALE3D		FVector(2.f, 2.f, 2.f)

#define LOCTEXT_NAMESPACE "EditorLightingBuildPromotionTests"

DEFINE_LOG_CATEGORY_STATIC(LogLightingTests, All, All);

namespace LightingTestHelpers
{
	// Searches through the list of actors in the level to find the actor that matches the desired name.
	// @CurrentLevel - The level to search through.
	// @ActorsName - The name of the actor to search for.
	bool DoesActorExistInTheLevel(const ULevel* CurrentLevel, const FString& ActorsName, const UClass* ActorToFind)
	{
		for (int32 i = 0; i < CurrentLevel->Actors.Num(); i++)
		{
			FString PossibleActorsName = CurrentLevel->Actors[i]->GetName();
			if (PossibleActorsName.Contains(ActorsName) && ActorToFind == CurrentLevel->Actors[i]->GetClass())
			{
				UE_LOGF(LogLightingTests, Log, "Found: %ls", *PossibleActorsName);
				return true;
			}
		}
		return false;
	}

	void GetActorCurrentLocation(const ULevel* CurrentLevel, const FString& ActorsName, FVector& OutActorLocation)
	{
		for (int32 i = 0; i < CurrentLevel->Actors.Num(); i++)
		{
			FString PossibleActorsName = CurrentLevel->Actors[i]->GetName();
			if (PossibleActorsName.Contains(ActorsName))
			{
				OutActorLocation = CurrentLevel->Actors[i]->GetActorLocation();
				break;
			}
		}
	}

	void GetActorCurrentRotation(const ULevel* CurrentLevel, const FString ActorsName, FRotator& OutActorRotation)
	{
		for (int32 i = 0; i < CurrentLevel->Actors.Num(); i++)
		{
			FString PossibleActorsName = CurrentLevel->Actors[i]->GetName();
			if (PossibleActorsName.Contains(ActorsName))
			{
				OutActorRotation = CurrentLevel->Actors[i]->GetActorRotation();
				break;
			}
		}
	}

	void GetActorCurrentScale3D(const ULevel* CurrentLevel, const FString ActorsName, FVector& OutActorScale3D)
	{
		for (int32 i = 0; i < CurrentLevel->Actors.Num(); i++)
		{
			FString PossibleActorsName = CurrentLevel->Actors[i]->GetName();
			if (PossibleActorsName.Contains(ActorsName))
			{
				OutActorScale3D = CurrentLevel->Actors[i]->GetActorScale3D();
				break;
			}
		}
	}

	/**
	* Sets an object property value by name
	* @param TargetObject - The object to modify
	* @param InVariableName - The name of the property
	*/
	void SetPropertyByName(UObject* TargetObject, const FString& InVariableName, const FString& NewValueString)
	{
		FProperty* FoundProperty = FindFProperty<FProperty>(TargetObject->GetClass(), *InVariableName);
		if (FoundProperty)
		{
			const FScopedTransaction PropertyChanged(LOCTEXT("PropertyChanged", "Object Property Change"));

			TargetObject->Modify();

			TargetObject->PreEditChange(FoundProperty);
			FoundProperty->ImportText_Direct(*NewValueString, FoundProperty->ContainerPtrToValuePtr<uint8>(TargetObject), TargetObject, 0);
			FPropertyChangedEvent PropertyChangedEvent(FoundProperty, EPropertyChangeType::ValueSet);
			TargetObject->PostEditChangeProperty(PropertyChangedEvent);
		}
	}

	void SelectActorInLevel(AActor* ActorToSelect)
	{
		//Deselect everything and then select the actor.
		GEditor->SelectNone(false, true, false);
		GEditor->SelectActor(ActorToSelect, true, false, true);
	}
}

//////////////////////////////////////////////////////////////////////////
//Lighting Promotion Test
//////////////////////////////////////////////////////////////////////////

/**
* Lighting Promotion Test - Place a Point Light and move it to a new location.
*/
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLightingPromotionPointLightPlaceRotScaleTest, "System.Promotion.Editor.Lighting.Place Scale Rotate", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FLightingPromotionPointLightPlaceRotScaleTest::RunTest(const FString& Parameters)
{
	//** SETUP **//
	// Create the world.
	UWorld* World = FAutomationEditorCommonUtils::CreateNewMap();
	ULevel* CurrentLevel = World->GetCurrentLevel();
	// Test Summary
	AddInfo(TEXT("Place, Scale, and Rotate.\n- A Point light is placed into the world.\n- The light is moved.\n- The light is rotated.\n- The light is scaled up."));

	if (!LightingTestHelpers::DoesActorExistInTheLevel(CurrentLevel, TEXT("PointLight"), APointLight::StaticClass()))
	{
		//** TEST **//
		// Add a point light to the level.
		APointLight* PointLight = Cast<APointLight>(GEditor->AddActor(World->GetCurrentLevel(), APointLight::StaticClass(), FTransform()));
		// Set the actors location, rotation, and scale3D.
		PointLight->SetActorLocation(POINT_LIGHT_UPDATED_LOCATION);
		PointLight->SetActorRotation(POINT_LIGHT_UPDATED_ROTATION);
		PointLight->SetActorScale3D(POINT_LIGHT_UPDATED_SCALE3D);

		//** VERIFY **//
		FVector CurrentLocation;
		LightingTestHelpers::GetActorCurrentLocation(CurrentLevel, PointLight->GetName(), CurrentLocation);
		FRotator CurrentRotation;
		LightingTestHelpers::GetActorCurrentRotation(CurrentLevel, PointLight->GetName(), CurrentRotation);
		FVector CurrentScale3D;
		LightingTestHelpers::GetActorCurrentScale3D(CurrentLevel, PointLight->GetName(), CurrentScale3D);
		bool RotationsAreEqual = CurrentRotation.Equals(POINT_LIGHT_UPDATED_ROTATION, 1);

		TestTrue(TEXT("The placed point light was not found."), LightingTestHelpers::DoesActorExistInTheLevel(CurrentLevel, PointLight->GetName(), PointLight->GetClass()));
		TestEqual(TEXT("The point light is not in correct location"), CurrentLocation, POINT_LIGHT_UPDATED_LOCATION);
		TestTrue(TEXT("The point light is not rotated correctly."), RotationsAreEqual);
		TestEqual(TEXT("The point light is not scaled correctly."), CurrentScale3D, POINT_LIGHT_UPDATED_SCALE3D);

		return true;
	}

	AddError(TEXT("A point light already exists in this level which will block the verification of a new point light."));
	return false;
}

/**
* Lighting Promotion Test - Modify a point lights properties.
*/
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLightingPromotionModifyProperties, "System.Promotion.Editor.Lighting.Modify Properties", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FLightingPromotionModifyProperties::RunTest(const FString& Parameters)
{
	//** SETUP **//
	UWorld* World = FAutomationEditorCommonUtils::CreateNewMap();
	ULevel* CurrentLevel = World->GetCurrentLevel();
	
	if (!LightingTestHelpers::DoesActorExistInTheLevel(CurrentLevel, TEXT("PointLight"), APointLight::StaticClass()))
	{
		//** TEST **//
		// Add a point light to the level.
		APointLight* PointLight = Cast<APointLight>(GEditor->AddActor(World->GetCurrentLevel(), APointLight::StaticClass(), FTransform()));
		// Modify the Lights Intensity, Light Color, and Attenuation Radius using it's properties.
		LightingTestHelpers::SetPropertyByName(PointLight->PointLightComponent, TEXT("Intensity"), TEXT("1000.f"));
		LightingTestHelpers::SetPropertyByName(PointLight->PointLightComponent, TEXT("LightColor"), TEXT("(R=0,G=0,B=255)"));
		LightingTestHelpers::SetPropertyByName(PointLight->PointLightComponent, TEXT("AttenuationRadius"), TEXT("1024.f"));

		//** VERIFY **//
		TestEqual(TEXT("Light Brightness"), PointLight->PointLightComponent->Intensity, 1000.f);
		TestEqual(TEXT("Light Color"), PointLight->PointLightComponent->LightColor, FColor(0, 0, 255));
		TestEqual(TEXT("Light Attenuation Radius"), PointLight->PointLightComponent->AttenuationRadius, 1024.f);

		return true;
	}

	AddError(TEXT("A point light already exists in this level which will block the verification of a new point light."));
	return false;
}


/*
/ **
* Lighting Promotion Test - Duplicate/Copy Paste a point light.
* /
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLightingPromotionDuplicationTest, "System.Promotion.Editor.Lighting.Duplicate and Copy Paste", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FLightingPromotionDuplicationTest::RunTest(const FString& Parameters)
{
	UWorld* World = FAutomationEditorCommonUtils::CreateNewMap();
	ULevel* CurrentLevel = World->GetCurrentLevel();
	// Test Summary
	AddInfo(TEXT("Duplicate and Copy Paste\n- Duplicates a point light.\n- Copies and Pastes a point light."));

	GLevelEditorModeTools().ActivateDefaultMode();

	if (!LightingTestHelpers::DoesActorExistInTheLevel(CurrentLevel, TEXT("PointLight"), APointLight::StaticClass()))
	{
		// Add a point light to the level.
		APointLight* PointLight = Cast<APointLight>(GEditor->AddActor(World->GetCurrentLevel(), APointLight::StaticClass(), FTransform()));

		int32 NumberOfPointLights = 0;
		// Count the number of point lights in the level.
		for ( TActorIterator<APointLight> It(World); It; ++It )
		{
			NumberOfPointLights++;
		}

		// Make sure there's only one point light in the level.
		TestEqual(TEXT("The light count before copy/paste"), NumberOfPointLights, 1);

		// Deselect all and then Select the light
		LightingTestHelpers::SelectActorInLevel(PointLight);

		TestEqual(TEXT("Selected actor count"), GEditor->GetSelectedActorCount(), 1);

		TestEqual<APointLight*>(TEXT("PointLight is selected"), PointLight, GEditor->GetSelectedActors()->GetTop<APointLight>());

		// Copy and Paste.
		GEditor->CopySelectedActorsToClipboard(World, false);

		FText TransDescription = FText::FromString("Automation Test Copy/Paste");
		GEditor->PasteSelectedActorsFromClipboard(World, TransDescription, EPasteTo::PT_OriginalLocation);

		NumberOfPointLights = 0;
		// Count the number of point lights in the level.
		for ( TActorIterator<APointLight> It(World); It; ++It )
		{
			NumberOfPointLights++;
		}

		// We are expecting two point lights to be in the level now.
		TestEqual(TEXT("The light count after copy/paste"), NumberOfPointLights, 2);

		// Deselect all and then select a light
		LightingTestHelpers::SelectActorInLevel(PointLight);

		// Duplicate the light
		GEngine->Exec(World, TEXT("DUPLICATE"));

		NumberOfPointLights = 0;
		// Count the number of point lights in the level.
		for ( TActorIterator<APointLight> It(World); It; ++It )
		{
			NumberOfPointLights++;
		}
		
		// We are expecting three point lights to be in the level now.
		TestEqual(TEXT("The light count after duplication"), NumberOfPointLights, 3);

		return true;
	}

	AddError(TEXT("A point light already exists in this level which would dirty the test results."));
	return false;
}*/


//////////////////////////////////////////////////////////////////////////
//Lighting Tests
//////////////////////////////////////////////////////////////////////////

/**
* Place a point light in the world with it's default settings
* True if the light exists in the levels actor array, otherwise False.
*/
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLightPointLightPlacement, "Editor.Lighting.Point Light.Placement", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FLightPointLightPlacement::RunTest(const FString& Parameters)
{
	//** SETUP **//
	// Create a new level.
	UWorld* World = FAutomationEditorCommonUtils::CreateNewMap();
	ULevel* CurrentLevel = World->GetCurrentLevel();
	// Light Setup
	APointLight* PointLight = nullptr;
	const FTransform Transform;
	
	if (!LightingTestHelpers::DoesActorExistInTheLevel(CurrentLevel, TEXT("PointLight"), APointLight::StaticClass()))
	{
		//** TEST **//
		// Add a point light to the level.
		PointLight = Cast<APointLight>(GEditor->AddActor(World->GetCurrentLevel(), APointLight::StaticClass(), Transform));

		//** VERIFY **//
		TestTrue(TEXT("The placed point light was not found."), LightingTestHelpers::DoesActorExistInTheLevel(CurrentLevel, PointLight->GetName(), PointLight->GetClass()));
		return true;
	}

	AddError(TEXT("A point light already exists in this level which will block the verification of a new point light."));
	return false;
}

/**
* Place a point light in the world with it's default settings
* True if the light exists in the levels actor array, otherwise False.
*/
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLightPointLightSetLocation, "Editor.Lighting.Point Light.Set Location", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FLightPointLightSetLocation::RunTest(const FString& Parameters)
{
	//** SETUP **//
	// Create a new level.
	UWorld* World = FAutomationEditorCommonUtils::CreateNewMap();
	ULevel* CurrentLevel = World->GetCurrentLevel();
	// Light Setup
	APointLight* PointLight = nullptr;
	const FTransform Transform;

	if (!LightingTestHelpers::DoesActorExistInTheLevel(CurrentLevel, TEXT("PointLight"), APointLight::StaticClass()))
	{
		//** TEST **//
		// Add a point light to the level.
		PointLight = Cast<APointLight>(GEditor->AddActor(World->GetCurrentLevel(), APointLight::StaticClass(), Transform));
		PointLight->SetActorLocation(POINT_LIGHT_UPDATED_LOCATION);

		//** VERIFY **//
		FVector CurrentLocation;
		LightingTestHelpers::GetActorCurrentLocation(CurrentLevel, PointLight->GetName(), CurrentLocation);

		TestEqual(TEXT("The point light is not in correct location"), CurrentLocation, POINT_LIGHT_UPDATED_LOCATION);
		return true;
	}

	AddError(TEXT("A point light already exists in this level which will block the verification of a new point light."));
	return false;
}

#undef LOCTEXT_NAMESPACE
