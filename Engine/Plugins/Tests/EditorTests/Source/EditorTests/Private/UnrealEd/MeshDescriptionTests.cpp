// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"
#include "Misc/AutomationTest.h"
#include "Modules/ModuleManager.h"
#include "Engine/StaticMesh.h"
#include "Materials/Material.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "MeshBuilder.h"
#include "RawMesh.h"
#include "MeshUtilities.h"

//////////////////////////////////////////////////////////////////////////

/**
* FMeshDescriptionAutomationTest
* Test that verify the MeshDescription functionalities. (Creation, modification, conversion to/from FRawMesh, render build)
* The tests will create some transient geometry using the mesh description API
* Cannot be run in a commandlet as it executes code that routes through Slate UI.
*/
IMPLEMENT_COMPLEX_AUTOMATION_TEST(FMeshDescriptionAutomationTest, "Editor.Meshes.MeshDescription", (EAutomationTestFlags::EditorContext | EAutomationTestFlags::NonNullRHI | EAutomationTestFlags::EngineFilter))

#define ConversionTestData 1
#define NTB_TestData 2

class FMeshDescriptionTest
{
public:
	FMeshDescriptionTest(const FString& InBeautifiedNames, const FString& InTestData)
		: BeautifiedNames(InBeautifiedNames)
		, TestData(InTestData)
	{}

	FString BeautifiedNames;
	FString TestData;
	bool Execute(FAutomationTestExecutionInfo& ExecutionInfo);
	bool ConversionTest(FAutomationTestExecutionInfo& ExecutionInfo);
	bool NTBTest(FAutomationTestExecutionInfo& ExecutionInfo);
private:
	bool CompareRawMesh(const FString& AssetName, FAutomationTestExecutionInfo& ExecutionInfo, const FRawMesh& ReferenceRawMesh, const FRawMesh& ResultRawMesh) const;
	bool CompareMeshDescription(const FString& AssetName, FAutomationTestExecutionInfo& ExecutionInfo, const FMeshDescription& ReferenceMeshDescription, const FMeshDescription& MeshDescription) const;
};

class FMeshDescriptionTests
{
public:
	static FMeshDescriptionTests& GetInstance()
	{
		static FMeshDescriptionTests Instance;
		return Instance;
	}

	void ClearTests()
	{
		AllTests.Empty();
	}

	bool ExecTest(int32 TestKey, FAutomationTestExecutionInfo& ExecutionInfo)
	{
		check(AllTests.Contains(TestKey));
		FMeshDescriptionTest& Test = AllTests[TestKey];
		return Test.Execute(ExecutionInfo);
	}

	bool AddTest(const FMeshDescriptionTest& MeshDescriptionTest)
	{
		if (!MeshDescriptionTest.TestData.IsNumeric())
		{
			return false;
		}
		int32 TestID = FPlatformString::Atoi(*(MeshDescriptionTest.TestData));
		check(!AllTests.Contains(TestID));
		AllTests.Add(TestID, MeshDescriptionTest);
		return true;
	}

private:
	TMap<int32, FMeshDescriptionTest> AllTests;
	FMeshDescriptionTests() {}
};

/**
* Requests a enumeration of all sample assets to import
*/
void FMeshDescriptionAutomationTest::GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const
{
	FMeshDescriptionTests::GetInstance().ClearTests();
	//Create conversion test
	FMeshDescriptionTest ConversionTest(TEXT("Conversion test data"), FString::FromInt(ConversionTestData));
	FMeshDescriptionTests::GetInstance().AddTest(ConversionTest);
	OutBeautifiedNames.Add(ConversionTest.BeautifiedNames);
	OutTestCommands.Add(ConversionTest.TestData);
	//Create NormalTangentBinormal test
/*	FMeshDescriptionTest NTB_Test(TEXT("Normals Tangents and Binormals test"), FString::FromInt(NTB_TestData));
	FMeshDescriptionTests::GetInstance().AddTest(NTB_Test);
	OutBeautifiedNames.Add(NTB_Test.BeautifiedNames);
	OutTestCommands.Add(NTB_Test.TestData);
*/
}

/**
* Execute the generic import test
*
* @param Parameters - Should specify the asset to import
* @return	TRUE if the test was successful, FALSE otherwise
*/
bool FMeshDescriptionAutomationTest::RunTest(const FString& Parameters)
{
	if (!Parameters.IsNumeric())
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("Wrong parameter for mesh description test parameter should be a number: [%s]"), *Parameters)));
		return false;
	}
	int32 TestID = FPlatformString::Atoi(*Parameters);
	return FMeshDescriptionTests::GetInstance().ExecTest(TestID, ExecutionInfo);
}

bool FMeshDescriptionTest::Execute(FAutomationTestExecutionInfo& ExecutionInfo)
{
	if (!TestData.IsNumeric())
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("Wrong parameter for mesh description test parameter should be a number: [%s]"), *TestData)));
		return false;
	}
	int32 TestID = FPlatformString::Atoi(*TestData);
	bool bSuccess = false;
	switch (TestID)
	{
	case ConversionTestData:
		bSuccess = ConversionTest(ExecutionInfo);
		break;
	case NTB_TestData:
		bSuccess = NTBTest(ExecutionInfo);
		break;
	}
	return bSuccess;
}

template<typename T>
void StructureArrayCompareFullPrecision(const FString& ConversionName, const FString& AssetName, FAutomationTestExecutionInfo& ExecutionInfo, bool& bIsSame, const FString& VectorArrayName, const TArray<T>& ReferenceArray, const TArray<T>& ResultArray)
{
	if (ReferenceArray.Num() != ResultArray.Num())
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("The %s conversion %s is not lossless, %s count is different. %s count expected [%d] result [%d]"),
			*AssetName,
			*ConversionName,
			*VectorArrayName,
			*VectorArrayName,
			ReferenceArray.Num(),
			ResultArray.Num())));
		bIsSame = false;
	}
	else
	{
		for (int32 VertexIndex = 0; VertexIndex < ReferenceArray.Num(); ++VertexIndex)
		{
			if (ReferenceArray[VertexIndex] != ResultArray[VertexIndex])
			{
				ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("The %s conversion %s is not lossless, %s array is different. Array index [%d] expected %s [%s] result [%s]"),
					*AssetName,
					*ConversionName,
					*VectorArrayName,
					VertexIndex,
					*VectorArrayName,
					*ReferenceArray[VertexIndex].ToString(),
					*ResultArray[VertexIndex].ToString())));
				bIsSame = false;
				break;
			}
		}
	}
}

template<typename T>
void StructureArrayCompare(const FString& ConversionName, const FString& AssetName, FAutomationTestExecutionInfo& ExecutionInfo, bool& bIsSame, const FString& VectorArrayName, const TArray<T>& ReferenceArray, const TArray<T>& ResultArray)
{
	if (ReferenceArray.Num() != ResultArray.Num())
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("The %s conversion %s is not lossless, %s count is different. %s count expected [%d] result [%d]"),
			*AssetName,
			*ConversionName,
			*VectorArrayName,
			*VectorArrayName,
			ReferenceArray.Num(),
			ResultArray.Num())));
		bIsSame = false;
	}
	else
	{
		for (int32 VertexIndex = 0; VertexIndex < ReferenceArray.Num(); ++VertexIndex)
		{
			if (!ReferenceArray[VertexIndex].Equals(ResultArray[VertexIndex]))
			{
				ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("The %s conversion %s is not lossless, %s array is different. Array index [%d] expected %s [%s] result [%s]"),
					*AssetName,
					*ConversionName,
					*VectorArrayName,
					VertexIndex,
					*VectorArrayName,
					*ReferenceArray[VertexIndex].ToString(),
					*ResultArray[VertexIndex].ToString())));
				bIsSame = false;
				break;
			}
		}
	}
}

template<typename T>
void NumberArrayCompare(const FString& ConversionName, const FString& AssetName, FAutomationTestExecutionInfo& ExecutionInfo, bool& bIsSame, const FString& VectorArrayName, const TArray<T>& ReferenceArray, const TArray<T>& ResultArray)
{
	if (ReferenceArray.Num() != ResultArray.Num())
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("The %s conversion %s is not lossless, %s count is different. %s count expected [%d] result [%d]"),
			*AssetName,
			*ConversionName,
			*VectorArrayName,
			*VectorArrayName,
			ReferenceArray.Num(),
			ResultArray.Num())));
		bIsSame = false;
	}
	else
	{
		for (int32 VertexIndex = 0; VertexIndex < ReferenceArray.Num(); ++VertexIndex)
		{
			if (ReferenceArray[VertexIndex] != ResultArray[VertexIndex])
			{
				ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("The %s conversion %s is not lossless, %s array is different. Array index [%d] expected %s [%d] result [%d]"),
					*AssetName,
					*ConversionName,
					*VectorArrayName,
					VertexIndex,
					*VectorArrayName,
					ReferenceArray[VertexIndex],
					ResultArray[VertexIndex])));
				bIsSame = false;
				break;
			}
		}
	}
}

bool FMeshDescriptionTest::CompareRawMesh(const FString& AssetName, FAutomationTestExecutionInfo& ExecutionInfo, const FRawMesh& ReferenceRawMesh, const FRawMesh& ResultRawMesh) const
{
	//////////////////////////////////////////////////////////////////////////
	// Do the comparison
	bool bAllSame = true;

	FString ConversionName = TEXT("RawMesh to MeshDescription to RawMesh");

	//Positions
	StructureArrayCompare<FVector3f>(ConversionName, AssetName, ExecutionInfo, bAllSame, TEXT("vertex positions"), ReferenceRawMesh.VertexPositions, ResultRawMesh.VertexPositions);

	//Normals
	StructureArrayCompare<FVector3f>(ConversionName, AssetName, ExecutionInfo, bAllSame, TEXT("vertex instance normals"), ReferenceRawMesh.WedgeTangentZ, ResultRawMesh.WedgeTangentZ);

	//Tangents
	StructureArrayCompare<FVector3f>(ConversionName, AssetName, ExecutionInfo, bAllSame, TEXT("vertex instance tangents"), ReferenceRawMesh.WedgeTangentX, ResultRawMesh.WedgeTangentX);

	//BiNormal
	StructureArrayCompare<FVector3f>(ConversionName, AssetName, ExecutionInfo, bAllSame, TEXT("vertex instance binormals"), ReferenceRawMesh.WedgeTangentY, ResultRawMesh.WedgeTangentY);

	//Colors --- FColor do not have Equals, so let use the full precision (FColor use integer anyway)
	StructureArrayCompareFullPrecision<FColor>(ConversionName, AssetName, ExecutionInfo, bAllSame, TEXT("vertex instance colors"), ReferenceRawMesh.WedgeColors, ResultRawMesh.WedgeColors);

	//Uvs
	for (int32 UVIndex = 0; UVIndex < MAX_MESH_TEXTURE_COORDS; ++UVIndex)
	{
		FString UVIndexName = FString::Printf(TEXT("vertex instance UVs(%d)"), UVIndex);
		StructureArrayCompare<FVector2f>(ConversionName, AssetName, ExecutionInfo, bAllSame, UVIndexName, ReferenceRawMesh.WedgeTexCoords[UVIndex], ResultRawMesh.WedgeTexCoords[UVIndex]);
	}
	
	//Indices
	NumberArrayCompare<uint32>(ConversionName, AssetName, ExecutionInfo, bAllSame, TEXT("vertex positions"), ReferenceRawMesh.WedgeIndices, ResultRawMesh.WedgeIndices);

	//Face
	NumberArrayCompare<int32>(ConversionName, AssetName, ExecutionInfo, bAllSame, TEXT("face material"), ReferenceRawMesh.FaceMaterialIndices, ResultRawMesh.FaceMaterialIndices);

	//Smoothing Mask
	NumberArrayCompare<uint32>(ConversionName, AssetName, ExecutionInfo, bAllSame, TEXT("smoothing mask"), ReferenceRawMesh.FaceSmoothingMasks, ResultRawMesh.FaceSmoothingMasks);

	return bAllSame;
}


template <typename T> FString AttributeValueAsString(const T& Value) { return LexToString(Value); }
FString AttributeValueAsString(const FVector3f& Value) { return Value.ToString(); }
FString AttributeValueAsString(const FVector2f& Value) { return Value.ToString(); }
FString AttributeValueAsString(const FVector4f& Value) { return Value.ToString(); }

template<typename T, typename U>
void MeshDescriptionAttributeArrayCompare(const FString& ConversionName, const FString& AssetName, FAutomationTestExecutionInfo& ExecutionInfo, bool& bIsSame, const U& ElementIterator, const FString& ArrayName, const T ReferenceArray, const T ResultArray)
{
	if (ReferenceArray.GetNumElements() != ResultArray.GetNumElements())
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("The %s conversion %s is not lossless, %s count is different. %s count expected [%d] result [%d]"),
			*AssetName,
			*ConversionName,
			*ArrayName,
			*ArrayName,
			ReferenceArray.GetNumElements(),
			ResultArray.GetNumElements())));
		bIsSame = false;
	}
	else if (ReferenceArray.GetNumChannels() != ResultArray.GetNumChannels())
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("The %s conversion %s is not lossless, %s channel count is different. %s channel count expected [%d] result [%d]"),
			*AssetName,
			*ConversionName,
			*ArrayName,
			*ArrayName,
			ReferenceArray.GetNumChannels(),
			ResultArray.GetNumChannels())));
		bIsSame = false;
	}
	else
	{
		int32 NumDifferent = 0;
		for (int32 Index = 0; Index < ReferenceArray.GetNumChannels(); ++Index)
		{
			for (auto ElementID : ElementIterator.GetElementIDs())
			{
				if (ReferenceArray.Get(ElementID, Index) != ResultArray.Get(ElementID, Index))
				{
					NumDifferent++;
					if (NumDifferent < 5)
					{
						ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("The %s conversion %s is not lossless, %s array is different. Element [%d] of attribute index [%d] expected %s [%s] result [%s]"),
							*AssetName,
							*ConversionName,
							*ArrayName,
							ElementID.GetValue(),
							Index,
							*ArrayName,
							*AttributeValueAsString(ReferenceArray.Get(ElementID, Index)),
							*AttributeValueAsString(ResultArray.Get(ElementID, Index)))));
					}
					else
					{
						ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, TEXT("More than 5 unequal elements - silencing")));
						break;
					}
					bIsSame = false;
				}
			}
		}
	}
}

bool FMeshDescriptionTest::CompareMeshDescription(const FString& AssetName, FAutomationTestExecutionInfo& ExecutionInfo, const FMeshDescription& ReferenceMeshDescription, const FMeshDescription& MeshDescription) const
{
	//////////////////////////////////////////////////////////////////////////
	//Gather the reference data
	FStaticMeshConstAttributes ReferenceAttributes(ReferenceMeshDescription);
	TVertexAttributesConstRef<FVector3f> ReferenceVertexPositions = ReferenceAttributes.GetVertexPositions();
	TVertexInstanceAttributesConstRef<FVector3f> ReferenceVertexInstanceNormals = ReferenceAttributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesConstRef<FVector3f> ReferenceVertexInstanceTangents = ReferenceAttributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesConstRef<float> ReferenceVertexInstanceBinormalSigns = ReferenceAttributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesConstRef<FVector4f> ReferenceVertexInstanceColors = ReferenceAttributes.GetVertexInstanceColors();
	TVertexInstanceAttributesConstRef<FVector2f> ReferenceVertexInstanceUVs = ReferenceAttributes.GetVertexInstanceUVs();
	TEdgeAttributesConstRef<bool> ReferenceEdgeHardnesses = ReferenceAttributes.GetEdgeHardnesses();
	TPolygonGroupAttributesConstRef<FName> ReferencePolygonGroupMaterialName = ReferenceAttributes.GetPolygonGroupMaterialSlotNames();


	//////////////////////////////////////////////////////////////////////////
	//Gather the result data
	FStaticMeshConstAttributes ResultAttributes(MeshDescription);
	TVertexAttributesConstRef<FVector3f> ResultVertexPositions = ResultAttributes.GetVertexPositions();
	TVertexInstanceAttributesConstRef<FVector3f> ResultVertexInstanceNormals = ResultAttributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesConstRef<FVector3f> ResultVertexInstanceTangents = ResultAttributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesConstRef<float> ResultVertexInstanceBinormalSigns = ResultAttributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesConstRef<FVector4f> ResultVertexInstanceColors = ResultAttributes.GetVertexInstanceColors();
	TVertexInstanceAttributesConstRef<FVector2f> ResultVertexInstanceUVs = ResultAttributes.GetVertexInstanceUVs();
	TEdgeAttributesConstRef<bool> ResultEdgeHardnesses = ResultAttributes.GetEdgeHardnesses();
	TPolygonGroupAttributesConstRef<FName> ResultPolygonGroupMaterialName = ResultAttributes.GetPolygonGroupMaterialSlotNames();

	
	//////////////////////////////////////////////////////////////////////////
	// Do the comparison
	bool bAllSame = true;

	FString ConversionName = TEXT("MeshDescription to RawMesh to MeshDescription");

	//Positions
	MeshDescriptionAttributeArrayCompare(ConversionName, AssetName, ExecutionInfo, bAllSame, ReferenceMeshDescription.Vertices(), TEXT("vertex positions"), ReferenceVertexPositions, ResultVertexPositions);

	//Normals
	MeshDescriptionAttributeArrayCompare(ConversionName, AssetName, ExecutionInfo, bAllSame, ReferenceMeshDescription.VertexInstances(), TEXT("vertex instance normals"), ReferenceVertexInstanceNormals, ResultVertexInstanceNormals);

	//Tangents
	MeshDescriptionAttributeArrayCompare(ConversionName, AssetName, ExecutionInfo, bAllSame, ReferenceMeshDescription.VertexInstances(), TEXT("vertex instance tangents"), ReferenceVertexInstanceTangents, ResultVertexInstanceTangents);

	//BiNormal signs
	MeshDescriptionAttributeArrayCompare(ConversionName, AssetName, ExecutionInfo, bAllSame, ReferenceMeshDescription.VertexInstances(), TEXT("vertex instance binormals"), ReferenceVertexInstanceBinormalSigns, ResultVertexInstanceBinormalSigns);

	//Colors
	MeshDescriptionAttributeArrayCompare(ConversionName, AssetName, ExecutionInfo, bAllSame, ReferenceMeshDescription.VertexInstances(), TEXT("vertex instance colors"), ReferenceVertexInstanceColors, ResultVertexInstanceColors);

	//Uvs
	MeshDescriptionAttributeArrayCompare(ConversionName, AssetName, ExecutionInfo, bAllSame, ReferenceMeshDescription.VertexInstances(), TEXT("vertex instance UVs"), ReferenceVertexInstanceUVs, ResultVertexInstanceUVs);

	//Edges
	//We do not use a template since we need to check the connected polygon count to validate a false comparison
	if (ReferenceEdgeHardnesses.GetNumElements() != ResultEdgeHardnesses.GetNumElements())
	{
		ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("The %s conversion MeshDescription to RawMesh to MeshDescription is not lossless, Edge count is different. Edges count expected [%d] result [%d]"),
			*AssetName,
			ReferenceEdgeHardnesses.GetNumElements(),
			ResultEdgeHardnesses.GetNumElements())));
		bAllSame = false;
	}
	else
	{
		for (const FEdgeID EdgeID : MeshDescription.Edges().GetElementIDs())
		{
			if (ReferenceEdgeHardnesses[EdgeID] != ResultEdgeHardnesses[EdgeID])
			{
				//Make sure it is not an external edge (only one polygon connected) since it is impossible to retain this information in a smoothing group.
				//External edge hardnesses has no impact on the normal calculation. It is useful only when editing meshes.
				const TArray<FPolygonID>& EdgeConnectedPolygons = MeshDescription.GetEdgeConnectedPolygons(EdgeID);
				if (EdgeConnectedPolygons.Num() > 1)
				{
					ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("The %s conversion to RawMesh is not lossless, Edge hardnesses array is different. EdgeID [%d] expected hardnesse [%s] result [%s]"),
						*AssetName,
						EdgeID.GetValue(),
						(ReferenceEdgeHardnesses[EdgeID] ? TEXT("true") : TEXT("false")),
						(ResultEdgeHardnesses[EdgeID] ? TEXT("true") : TEXT("false")))));
					bAllSame = false;
				}
				break;
			}
		}
	}

	//Polygon group ID
	MeshDescriptionAttributeArrayCompare(ConversionName, AssetName, ExecutionInfo, bAllSame, ReferenceMeshDescription.PolygonGroups(), TEXT("PolygonGroup Material Name"), ReferencePolygonGroupMaterialName, ResultPolygonGroupMaterialName);

	return bAllSame;
}

bool FMeshDescriptionTest::ConversionTest(FAutomationTestExecutionInfo& ExecutionInfo)
{
	TArray<FString> AssetNames;
	AssetNames.Add(TEXT("Cone_1"));
	AssetNames.Add(TEXT("Cone_2"));
	AssetNames.Add(TEXT("Cube"));
	AssetNames.Add(TEXT("Patch_1"));
	AssetNames.Add(TEXT("Patch_2"));
	AssetNames.Add(TEXT("Patch_3"));
	AssetNames.Add(TEXT("Patch_4"));
	AssetNames.Add(TEXT("Patch_5"));
	AssetNames.Add(TEXT("Pentagone"));
	AssetNames.Add(TEXT("Sphere_1"));
	AssetNames.Add(TEXT("Sphere_2"));
	AssetNames.Add(TEXT("Sphere_3"));
	AssetNames.Add(TEXT("Torus_1"));
	AssetNames.Add(TEXT("Torus_2"));

	bool bAllSame = true;
	for (const FString& AssetName : AssetNames)
	{
		FString FullAssetName = TEXT("/Game/Tests/MeshDescription/") + AssetName + TEXT(".") + AssetName;
		UStaticMesh* AssetMesh = LoadObject<UStaticMesh>(nullptr, *FullAssetName, nullptr, LOAD_None, nullptr);

		if (AssetMesh != nullptr)
		{
			AssetMesh->BuildCacheAutomationTestGuid = FGuid::NewGuid();
			TMap<FName, int32> MaterialMap;
			TMap<int32, FName> MaterialMapInverse;
			for (int32 MaterialIndex = 0; MaterialIndex < AssetMesh->GetStaticMaterials().Num(); ++MaterialIndex)
			{
				MaterialMap.Add(AssetMesh->GetStaticMaterials()[MaterialIndex].ImportedMaterialSlotName, MaterialIndex);
				MaterialMapInverse.Add(MaterialIndex, AssetMesh->GetStaticMaterials()[MaterialIndex].ImportedMaterialSlotName);
			}

			//MeshDescription to RawMesh to MeshDescription
			for(int32 LodIndex = 0; LodIndex < AssetMesh->GetNumSourceModels(); ++LodIndex)
			{
				const FMeshDescription* ReferenceAssetMesh = AssetMesh->GetMeshDescription(LodIndex);
				if (ReferenceAssetMesh == nullptr)
				{
					check(LodIndex != 0);
					continue;
				}
				//Create a temporary Mesh Description
				FMeshDescription ResultAssetMesh(*ReferenceAssetMesh);
				//Convert MeshDescription to FRawMesh
				FRawMesh RawMesh;
				FStaticMeshOperations::ConvertToRawMesh(ResultAssetMesh, RawMesh, MaterialMap);
				//Convert back the FRawmesh
				FStaticMeshOperations::ConvertFromRawMesh(RawMesh, ResultAssetMesh, MaterialMapInverse);
				if (!CompareMeshDescription(AssetName, ExecutionInfo, *ReferenceAssetMesh, ResultAssetMesh))
				{
					bAllSame = false;
				}
			}

			//RawMesh to MeshDescription to RawMesh
			for (int32 LodIndex = 0; LodIndex < AssetMesh->GetNumSourceModels(); ++LodIndex)
			{
				if (!AssetMesh->GetSourceModel(LodIndex).IsMeshDescriptionValid())
				{
					check(LodIndex != 0);
					continue;
				}
				FRawMesh ReferenceRawMesh;
				AssetMesh->GetSourceModel(LodIndex).LoadRawMesh(ReferenceRawMesh);
				FRawMesh ResultRawMesh;
				AssetMesh->GetSourceModel(LodIndex).LoadRawMesh(ResultRawMesh);
				//Create a temporary Mesh Description
				FMeshDescription MeshDescription;
				FStaticMeshAttributes(MeshDescription).Register();
				FStaticMeshOperations::ConvertFromRawMesh(ResultRawMesh, MeshDescription, MaterialMapInverse);
				//Convert back the FRawmesh
				FStaticMeshOperations::ConvertToRawMesh(MeshDescription, ResultRawMesh, MaterialMap);
				if (!CompareRawMesh(AssetName, ExecutionInfo, ReferenceRawMesh, ResultRawMesh))
				{
					bAllSame = false;
				}
			}
		}
	}
	return bAllSame;
}

bool FMeshDescriptionTest::NTBTest(FAutomationTestExecutionInfo& ExecutionInfo)
{
	TArray<FString> AssetNames;
	AssetNames.Add(TEXT("Cone_1"));
	AssetNames.Add(TEXT("Cone_2"));
	AssetNames.Add(TEXT("Cube"));
	AssetNames.Add(TEXT("Patch_1"));
	AssetNames.Add(TEXT("Patch_2"));
	AssetNames.Add(TEXT("Patch_3"));
	AssetNames.Add(TEXT("Patch_4"));
	AssetNames.Add(TEXT("Patch_5"));
	AssetNames.Add(TEXT("Pentagone"));
	AssetNames.Add(TEXT("Sphere_1"));
	AssetNames.Add(TEXT("Sphere_2"));
	AssetNames.Add(TEXT("Sphere_3"));
	AssetNames.Add(TEXT("Torus_1"));
	AssetNames.Add(TEXT("Torus_2"));

	bool bAllSame = true;
	for (const FString& AssetName : AssetNames)
	{
		FString FullAssetName = TEXT("/Game/Tests/MeshDescription/") + AssetName + TEXT(".") + AssetName;
		UStaticMesh* AssetMesh = LoadObject<UStaticMesh>(nullptr, *FullAssetName, nullptr, LOAD_None, nullptr);

		if (AssetMesh == nullptr)
		{
			continue;
		}
		//Dirty the build
		AssetMesh->BuildCacheAutomationTestGuid = FGuid::NewGuid();
		FMeshDescription* OriginalMeshDescription = AssetMesh->GetMeshDescription(0);
		check(OriginalMeshDescription);

		// Take a copy, so changes made by the test don't make permanent changes to the mesh description
		// @todo Alexis: is this OK?
		FMeshDescription MeshDescription(*OriginalMeshDescription);

		FRawMesh RawMesh;
		AssetMesh->GetSourceModel(0).LoadRawMesh(RawMesh);

		FStaticMeshAttributes Attributes(MeshDescription);
		const TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
		const TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
		const TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
		const TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
		const TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
		int32 ExistingUVCount = VertexInstanceUVs.GetNumChannels();

		//Build the normals and tangent and compare the result
		FStaticMeshOperations::ComputeTriangleTangentsAndNormals(MeshDescription, SMALL_NUMBER);
		EComputeNTBsFlags ComputeNTBsFlags = EComputeNTBsFlags::Normals | EComputeNTBsFlags::Tangents;
		FStaticMeshOperations::ComputeTangentsAndNormals(MeshDescription, ComputeNTBsFlags);
		//FMeshDescriptionOperations::CreatePolygonNTB(MeshDescription, SMALL_NUMBER);
		//FMeshDescriptionOperations::CreateNormals(MeshDescription, FMeshDescriptionOperations::ETangentOptions::BlendOverlappingNormals, true);

		IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>(TEXT("MeshUtilities"));
		FMeshBuildSettings MeshBuildSettings;
		MeshBuildSettings.bRemoveDegenerates = true;
		MeshBuildSettings.bUseMikkTSpace = false;
		MeshUtilities.RecomputeTangentsAndNormalsForRawMesh(true, true, MeshBuildSettings, RawMesh);

		//The normals and tangents of both the meshdescription and the RawMesh should be equal to not break old data
		if(RawMesh.WedgeIndices.Num() != MeshDescription.VertexInstances().Num())
		{
			ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("Test: [Normals Tangents and Binormals test]    Asset: [%s]    Error: The number of vertex instances is not equal between FRawMesh [%d] and UMeshDescription [%d]."),
				*AssetName,
				RawMesh.WedgeIndices.Num(),
				MeshDescription.VertexInstances().Num())));
			continue;
		}
		int32 VertexInstanceIndex = 0;
		int32 TriangleIndex = 0;

		auto OutputError=[&ExecutionInfo, &AssetName](FString ErrorMessage)
		{
			ExecutionInfo.AddEvent(FAutomationEvent(EAutomationEventType::Error, FString::Printf(TEXT("Test: [Normals Tangents and Binormals test]    Asset: [%s]    Error: %s."),
				*AssetName,
				*ErrorMessage)));
		};
		bool bError = false;
		for (const FPolygonID PolygonID : MeshDescription.Polygons().GetElementIDs())
		{
			if (bError)
			{
				break;
			}
			const FPolygonGroupID& PolygonGroupID = MeshDescription.GetPolygonPolygonGroup(PolygonID);
			int32 PolygonIDValue = PolygonID.GetValue();
			TArrayView<const FTriangleID> TriangleIDs = MeshDescription.GetPolygonTriangles(PolygonID);
			for (const FTriangleID& TriangleID : TriangleIDs)
			{
				if (bError)
				{
					break;
				}
				for (int32 Corner = 0; Corner < 3 && bError == false; ++Corner)
				{
					uint32 WedgeIndex = TriangleIndex * 3 + Corner;
					const FVertexInstanceID VertexInstanceID = MeshDescription.GetTriangleVertexInstance(TriangleID, Corner);
					const int32 VertexInstanceIDValue = VertexInstanceID.GetValue();
					if (RawMesh.WedgeColors[WedgeIndex] != FLinearColor(VertexInstanceColors[VertexInstanceID]).ToFColor(true))
					{
						FString MeshDescriptionColor = FLinearColor(VertexInstanceColors[VertexInstanceID]).ToFColor(true).ToString();
						OutputError(FString::Printf(TEXT("Vertex color is different between MeshDescription [%s] and FRawMesh [%s].   Indice[%d]")
							, *MeshDescriptionColor
							, *(RawMesh.WedgeColors[WedgeIndex].ToString())
							, VertexInstanceIDValue));
						bError = true;
					}
					if (RawMesh.WedgeIndices[WedgeIndex] != MeshDescription.GetVertexInstanceVertex(VertexInstanceID).GetValue())
					{
						OutputError(FString::Printf(TEXT("Vertex index is different between MeshDescription [%d] and FRawMesh [%d].   Indice[%d]")
							, MeshDescription.GetVertexInstanceVertex(VertexInstanceID).GetValue()
							, RawMesh.WedgeIndices[WedgeIndex]
							, VertexInstanceIDValue));
						bError = true;
					}
					if (!RawMesh.WedgeTangentX[WedgeIndex].Equals(VertexInstanceTangents[VertexInstanceID], THRESH_NORMALS_ARE_SAME))
					{
						OutputError(FString::Printf(TEXT("Vertex tangent is different between MeshDescription [%s] and FRawMesh [%s].   Indice[%d]")
							, *(VertexInstanceTangents[VertexInstanceID].ToString())
							, *(RawMesh.WedgeTangentX[WedgeIndex].ToString())
							, VertexInstanceIDValue));
						bError = true;
					}
					if (!RawMesh.WedgeTangentY[WedgeIndex].Equals(FVector3f::CrossProduct(VertexInstanceNormals[VertexInstanceID], VertexInstanceTangents[VertexInstanceID]).GetSafeNormal() * VertexInstanceBinormalSigns[VertexInstanceID], THRESH_NORMALS_ARE_SAME))
					{
						FVector MeshDescriptionBinormalResult = (FVector)FVector3f::CrossProduct(VertexInstanceNormals[VertexInstanceID], VertexInstanceTangents[VertexInstanceID]).GetSafeNormal() * VertexInstanceBinormalSigns[VertexInstanceID];
						OutputError(FString::Printf(TEXT("Vertex binormal is different between MeshDescription [%s] and FRawMesh [%s].   Indice[%d]")
							, *(MeshDescriptionBinormalResult.ToString())
							, *(RawMesh.WedgeTangentY[WedgeIndex].ToString())
							, VertexInstanceIDValue));
						bError = true;
					}
					if (!RawMesh.WedgeTangentZ[WedgeIndex].Equals(VertexInstanceNormals[VertexInstanceID], THRESH_NORMALS_ARE_SAME))
					{
						OutputError(FString::Printf(TEXT("Vertex normal is different between MeshDescription [%s] and FRawMesh [%s].   Indice[%d]")
							, *(VertexInstanceNormals[VertexInstanceID].ToString())
							, *(RawMesh.WedgeTangentZ[WedgeIndex].ToString())
							, VertexInstanceIDValue));
						bError = true;
					}

					for (int32 UVIndex = 0; UVIndex < ExistingUVCount; ++UVIndex)
					{
						if (!RawMesh.WedgeTexCoords[UVIndex][WedgeIndex].Equals(VertexInstanceUVs.Get(VertexInstanceID, UVIndex), THRESH_UVS_ARE_SAME))
						{
							OutputError(FString::Printf(TEXT("Vertex Texture coordinnate is different between MeshDescription [%s] and FRawMesh [%s].   UVIndex[%d]  Indice[%d]")
								, *(VertexInstanceUVs.Get(VertexInstanceID, UVIndex).ToString())
								, *(RawMesh.WedgeTexCoords[UVIndex][WedgeIndex].ToString())
								, UVIndex
								, VertexInstanceIDValue));
							bError = true;
						}
					}
				}
				++TriangleIndex;
			}
		}
	}

	return bAllSame;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMeshDescriptionBuilderTest, "Editor.Meshes.MeshDescription.Builder", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FMeshDescriptionBuilderTest::RunTest(const FString& Parameters)
{
	FMeshDescription MeshDescription;
	TVertexAttributesRef<FVector3f> Positions = MeshDescription.GetVertexPositions();

	// Build a hexagonal cylinder

	const float Radius = 100.0f;
	const float Height = 200.0f;
	const int32 NumSides = 6;

	FVertexID TopVertexIDs[NumSides];
	FVertexID BottomVertexIDs[NumSides];

	FVertexInstanceID TopVertexInstanceIDs[NumSides];
	FVertexInstanceID SideTopVertexInstanceIDs[NumSides];
	FVertexInstanceID BottomVertexInstanceIDs[NumSides];
	FVertexInstanceID SideBottomVertexInstanceIDs[NumSides];

	FEdgeID SideEdgeIDs[NumSides];
	FPolygonID SidePolygonIDs[NumSides];
	FTriangleID TopTriangleIDs[NumSides];
	FTriangleID BottomTriangleIDs[NumSides];

	for (int32 Index = 0; Index < NumSides; ++Index)
	{
		// Create vertices at top and bottom

		const float X = Radius * FMath::Cos(PI * 2.0f * Index / NumSides);
		const float Y = Radius * FMath::Sin(PI * 2.0f * Index / NumSides);

		TopVertexIDs[Index] = MeshDescription.CreateVertex();
		Positions[TopVertexIDs[Index]] = FVector3f(X, Y, -Height);

		BottomVertexIDs[Index] = MeshDescription.CreateVertex();
		Positions[BottomVertexIDs[Index]] = FVector3f(X, Y, Height);

		// Each vertex has two vertex instances: one for the side polygons, and one for the top/bottom polygons

		TopVertexInstanceIDs[Index] = MeshDescription.CreateVertexInstance(TopVertexIDs[Index]);
		SideTopVertexInstanceIDs[Index] = MeshDescription.CreateVertexInstance(TopVertexIDs[Index]);

		BottomVertexInstanceIDs[Index] = MeshDescription.CreateVertexInstance(BottomVertexIDs[Index]);
		SideBottomVertexInstanceIDs[Index] = MeshDescription.CreateVertexInstance(BottomVertexIDs[Index]);
	}

	// Create central vertex at bottom, and associated instances

	FVertexID BottomCenterVertexID = MeshDescription.CreateVertex();
	Positions[BottomCenterVertexID] = FVector3f(0.0f, 0.0f, Height);

	FVertexInstanceID BottomCenterVertexInstanceID = MeshDescription.CreateVertexInstance(BottomCenterVertexID);

	// Test vertex / vertex instance status

	TestEqual("Vertex count", MeshDescription.Vertices().Num(), NumSides * 2 + 1);
	TestEqual("Vertex array size", MeshDescription.Vertices().GetArraySize(), NumSides * 2 + 1);
	TestEqual("Vertex position count", Positions.GetNumElements(), NumSides * 2 + 1);

	TestEqual("Vertex instance count", MeshDescription.VertexInstances().Num(), NumSides * 4 + 1);
	TestEqual("Vertex instance array size", MeshDescription.VertexInstances().GetArraySize(), NumSides * 4 + 1);

	for (int32 Index = 0; Index < NumSides; ++Index)
	{
		TestEqual("Vertex instances per vertex", MeshDescription.GetNumVertexVertexInstances(TopVertexIDs[Index]), 2);
		TestTrue("Vertex contains vertex instance", MeshDescription.GetVertexVertexInstanceIDs(TopVertexIDs[Index]).Contains(TopVertexInstanceIDs[Index]));
		TestTrue("Vertex contains vertex instance", MeshDescription.GetVertexVertexInstanceIDs(TopVertexIDs[Index]).Contains(SideTopVertexInstanceIDs[Index]));
		TestEqual("Vertex instances per vertex", MeshDescription.GetNumVertexVertexInstances(BottomVertexIDs[Index]), 2);
		TestTrue("Vertex contains vertex instance", MeshDescription.GetVertexVertexInstanceIDs(BottomVertexIDs[Index]).Contains(BottomVertexInstanceIDs[Index]));
		TestTrue("Vertex contains vertex instance", MeshDescription.GetVertexVertexInstanceIDs(BottomVertexIDs[Index]).Contains(SideBottomVertexInstanceIDs[Index]));
		TestEqual("Vertex instance parent", MeshDescription.GetVertexInstanceVertex(TopVertexInstanceIDs[Index]), TopVertexIDs[Index]);
		TestEqual("Vertex instance parent", MeshDescription.GetVertexInstanceVertex(SideTopVertexInstanceIDs[Index]), TopVertexIDs[Index]);
		TestEqual("Vertex instance parent", MeshDescription.GetVertexInstanceVertex(BottomVertexInstanceIDs[Index]), BottomVertexIDs[Index]);
		TestEqual("Vertex instance parent", MeshDescription.GetVertexInstanceVertex(SideBottomVertexInstanceIDs[Index]), BottomVertexIDs[Index]);
	}

	// Add edges for side polys

	for (int32 Index = 0; Index < NumSides; ++Index)
	{
		SideEdgeIDs[Index] = MeshDescription.CreateEdge(BottomVertexIDs[Index], TopVertexIDs[Index]);

		TestTrue("Vertex connected edges", MeshDescription.GetVertexConnectedEdgeIDs(TopVertexIDs[Index]).Contains(SideEdgeIDs[Index]));
		TestTrue("Vertex connected edges", MeshDescription.GetVertexConnectedEdgeIDs(BottomVertexIDs[Index]).Contains(SideEdgeIDs[Index]));
	}

	// Add polygon groups

	FPolygonGroupID SidePolygonGroupID = MeshDescription.CreatePolygonGroup();
	FPolygonGroupID TopPolygonGroupID = MeshDescription.CreatePolygonGroup();
	FPolygonGroupID BottomPolygonGroupID = MeshDescription.CreatePolygonGroup();

	// Now add side polygons

	for (int32 Index = 0; Index < NumSides; ++Index)
	{
		// Define a side polygon to create
		// We have already explicitly created the side edges, so we expect CreatePolygon to add two extra ones, top and bottom

		const FVertexInstanceID SidePolyVertexInstanceIDs[4] =
		{
			SideBottomVertexInstanceIDs[Index],
			SideBottomVertexInstanceIDs[(Index + 1) % NumSides],
			SideTopVertexInstanceIDs[(Index + 1) % NumSides],
			SideTopVertexInstanceIDs[Index]
		};

		TArray<FEdgeID> OutEdgeIDs;
		SidePolygonIDs[Index] = MeshDescription.CreatePolygon(SidePolygonGroupID, SidePolyVertexInstanceIDs, &OutEdgeIDs);

		TestEqual("Extra edges created", OutEdgeIDs.Num(), 2);
		TestEqual("Bottom edge vertex 0", MeshDescription.GetEdgeVertex(OutEdgeIDs[0], 0), BottomVertexIDs[Index]);
		TestEqual("Bottom edge vertex 1", MeshDescription.GetEdgeVertex(OutEdgeIDs[0], 1), BottomVertexIDs[(Index + 1) % NumSides]);
		TestEqual("Top edge vertex 0", MeshDescription.GetEdgeVertex(OutEdgeIDs[1], 0), TopVertexIDs[(Index + 1) % NumSides]);
		TestEqual("Top edge vertex 1", MeshDescription.GetEdgeVertex(OutEdgeIDs[1], 1), TopVertexIDs[Index]);

		TestEqual("Triangles per side poly", MeshDescription.GetNumPolygonTriangles(SidePolygonIDs[Index]), 2);

		TArray<FEdgeID, TInlineAllocator<1>> InternalEdgeIDs = MeshDescription.GetPolygonInternalEdges<TInlineAllocator<1>>(SidePolygonIDs[Index]);
		TestEqual("Number of internal edges", InternalEdgeIDs.Num(), 1);
	}

	// Top face is a single polygon

	TArray<FEdgeID> OutTopPolyEdgeIDs;
	FPolygonID TopPolygonID = MeshDescription.CreatePolygon(TopPolygonGroupID, TopVertexInstanceIDs, &OutTopPolyEdgeIDs);

	TestEqual("Extra edges created", OutTopPolyEdgeIDs.Num(), 0);
	
	TArray<FEdgeID, TInlineAllocator<NumSides - 3>> TopInternalEdgeIDs = MeshDescription.GetPolygonInternalEdges<TInlineAllocator<NumSides - 3>>(TopPolygonID);
	TestEqual("Number of internal edges", TopInternalEdgeIDs.Num(), NumSides - 3);

	// Bottom face is composed of individual triangles

	for (int32 Index = 0; Index < NumSides; ++Index)
	{
		const FVertexInstanceID BottomTriangle[3] =
		{
			BottomVertexInstanceIDs[(Index + 1) % NumSides],
			BottomVertexInstanceIDs[Index],
			BottomCenterVertexInstanceID
		};

		TArray<FEdgeID> OutBottomTriEdgeIDs;
		BottomTriangleIDs[Index] = MeshDescription.CreateTriangle(BottomPolygonGroupID, BottomTriangle, &OutBottomTriEdgeIDs);

		const int32 ExtraEdges = (Index == 0) ? 2 : (Index == NumSides - 1) ? 0 : 1;
		TestEqual("Extra bottom triangle edges", OutBottomTriEdgeIDs.Num(), ExtraEdges);
	}

	// Check totals correct

	TestEqual("Total triangle count", MeshDescription.Triangles().Num(), NumSides * 4 - 2);
	TestEqual("Total edge count", MeshDescription.Edges().Num(), NumSides * 6 - 3);
	TestEqual("Polygons in side group", MeshDescription.GetNumPolygonGroupPolygons(SidePolygonGroupID), NumSides);
	TestEqual("Polygons in top group", MeshDescription.GetNumPolygonGroupPolygons(TopPolygonGroupID), 1);
	TestEqual("Polygons in bottom group", MeshDescription.GetNumPolygonGroupPolygons(BottomPolygonGroupID), NumSides);

	// Check connections configured correctly

	for (int32 Index = 0; Index < NumSides; ++Index)
	{
		int32 PrevIndex = (Index + NumSides - 1) % NumSides;

		TestEqual("Side polygons connected to edge", MeshDescription.GetNumEdgeConnectedPolygons(SideEdgeIDs[Index]), 2);
		TestEqual("Side triangles connected to edge", MeshDescription.GetNumEdgeConnectedTriangles(SideEdgeIDs[Index]), 2);
		TestTrue("Check polygon connected to edge", MeshDescription.GetEdgeConnectedPolygons(SideEdgeIDs[Index]).Contains(SidePolygonIDs[Index]));
		TestTrue("Check polygon connected to edge", MeshDescription.GetEdgeConnectedPolygons(SideEdgeIDs[Index]).Contains(SidePolygonIDs[PrevIndex]));

		TestEqual("Side polygons connected to vertex instance", MeshDescription.GetNumVertexInstanceConnectedPolygons(SideTopVertexInstanceIDs[Index]), 2);
		TestEqual("Side triangles connected to vertex instance", MeshDescription.GetNumVertexInstanceConnectedTriangles(SideTopVertexInstanceIDs[Index]), 3);
		TestTrue("Check polygon connected to vertex instance", MeshDescription.GetVertexInstanceConnectedPolygons(SideTopVertexInstanceIDs[Index]).Contains(SidePolygonIDs[Index]));
		TestTrue("Check polygon connected to vertex instance", MeshDescription.GetVertexInstanceConnectedPolygons(SideTopVertexInstanceIDs[Index]).Contains(SidePolygonIDs[PrevIndex]));
	}

	// Check adjacency

	TArray<FPolygonID> AdjacentPolygonIDs = MeshDescription.GetPolygonAdjacentPolygons(SidePolygonIDs[0]);
	TestEqual("Adjacent polygon count", AdjacentPolygonIDs.Num(), 4);
	TestTrue("Check adjacency", AdjacentPolygonIDs.Contains(SidePolygonIDs[1]));
	TestTrue("Check adjacency", AdjacentPolygonIDs.Contains(SidePolygonIDs[NumSides - 1]));
	TestTrue("Check adjacency", AdjacentPolygonIDs.Contains(MeshDescription.GetTrianglePolygon(BottomTriangleIDs[0])));
	TestTrue("Check adjacency", AdjacentPolygonIDs.Contains(TopPolygonID));

	// Move some points and check retriangulation

	Positions[TopVertexIDs[0]] = FVector3f(Radius * 0.25f, 0.0f, -Height);	// create a concave top and bottom
	Positions[BottomVertexIDs[0]] = FVector3f(Radius * 0.25f, 0.0f, Height);

	// Get list of unique polygons connected to the vertices that have moved
	TArray<FPolygonID> ConnectedPolygons = MeshDescription.GetVertexConnectedPolygons(TopVertexIDs[0]);
	for (const FPolygonID& ConnectedPolygon : MeshDescription.GetVertexConnectedPolygons(BottomVertexIDs[0]))
	{
		ConnectedPolygons.AddUnique(ConnectedPolygon);
	}

	TestEqual("Number of polys to triangulate", ConnectedPolygons.Num(), 5);
	TestTrue("PolygonsToTriangulate", ConnectedPolygons.Contains(TopPolygonID));
	TestTrue("PolygonsToTriangulate", ConnectedPolygons.Contains(SidePolygonIDs[0]));
	TestTrue("PolygonsToTriangulate", ConnectedPolygons.Contains(SidePolygonIDs[NumSides - 1]));
	TestTrue("PolygonsToTriangulate", ConnectedPolygons.Contains(MeshDescription.GetTrianglePolygon(BottomTriangleIDs[0])));
	TestTrue("PolygonsToTriangulate", ConnectedPolygons.Contains(MeshDescription.GetTrianglePolygon(BottomTriangleIDs[NumSides - 1])));

	for (const FPolygonID& ConnectedPolygon : ConnectedPolygons)
	{
		MeshDescription.ComputePolygonTriangulation(ConnectedPolygon);
	}

	TArray<FEdgeID, TInlineAllocator<NumSides - 3>> NewTopInternalEdgeIDs = MeshDescription.GetPolygonInternalEdges<TInlineAllocator<NumSides - 3>>(TopPolygonID);
	TestEqual("Number of internal edges", NewTopInternalEdgeIDs.Num(), NumSides - 3);

	// Move polygon to another group

	MeshDescription.SetPolygonPolygonGroup(TopPolygonID, SidePolygonGroupID);
	TestEqual("New polygons in side group", MeshDescription.GetNumPolygonGroupPolygons(SidePolygonGroupID), NumSides + 1);
	TestEqual("New polygons in top group", MeshDescription.GetNumPolygonGroupPolygons(TopPolygonGroupID), 0);
	TestEqual("Top polygon group", MeshDescription.GetPolygonPolygonGroup(TopPolygonID), SidePolygonGroupID);

	// Delete a mesh element

	TArray<FEdgeID> TopFaceEdges = MeshDescription.GetPolygonPerimeterEdges(TopPolygonID);
	for (int32 Index = 0; Index < NumSides; ++Index)
	{
		const FEdgeID TopFaceEdge = TopFaceEdges[Index];
		TArray<FPolygonID> Polys = MeshDescription.GetEdgeConnectedPolygons(TopFaceEdge);
		TestEqual("Check number of edge connected polygons", Polys.Num(), 2);
		TestTrue("Check edge connection", Polys.Contains(TopPolygonID));
//		TestTrue("Check edge connection", Polys.Contains(SidePolygonIDs[Index]));
	}

	TArray<FEdgeID> OrphanedEdges;
	TArray<FVertexInstanceID> OrphanedVertexInstances;
	TArray<FPolygonGroupID> OrphanedPolygonGroups;
	MeshDescription.DeletePolygon(TopPolygonID, &OrphanedEdges, &OrphanedVertexInstances, &OrphanedPolygonGroups);

	TestEqual("Orphaned edges", OrphanedEdges.Num(), 0);
	TestEqual("Orphaned vertex instances", OrphanedVertexInstances.Num(), NumSides);
	TestEqual("Orphaned polygon groups", OrphanedPolygonGroups.Num(), 0);
	TestEqual("New edge count", MeshDescription.Edges().Num(), NumSides * 5);
	TestEqual("New edge array size", MeshDescription.Edges().GetArraySize(), NumSides * 6 - 3);

	for (int32 Index = 0; Index < NumSides; ++Index)
	{
		const FEdgeID TopFaceEdge = TopFaceEdges[Index];
		TArray<FPolygonID> Polys = MeshDescription.GetEdgeConnectedPolygons(TopFaceEdge);
		TestEqual("Check number of edge connected polygons", Polys.Num(), 1);
//		TestTrue("Check edge connection", Polys.Contains(SidePolygonIDs[Index]));
	}

	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMeshDescriptionArrayAttributeTest, "Editor.Meshes.MeshDescription.ArrayAttribute", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FMeshDescriptionArrayAttributeTest::RunTest(const FString& Parameters)
{
	FAttributesSetBase AttributesSet;
	AttributesSet.RegisterAttribute<int[]>("Test");

	TMeshAttributesArray<TArrayAttribute<int>> Attributes = AttributesSet.GetAttributesRef<TArrayAttribute<int>>("Test");
	TestTrue("AttributeRef valid", Attributes.IsValid());
	TestEqual("AttributeRef element count", Attributes.GetNumElements(), 0);
	TestEqual("AttributeRef channel count", Attributes.GetNumChannels(), 1);

	AttributesSet.Initialize(10);
	TestEqual("AttributeRef element count", Attributes.GetNumElements(), 10);

	TArrayAttribute<int> Element0 = Attributes.Get(0);
	TestEqual("Default element array size", Element0.Num(), 0);
	Element0.Add(42);
	TestEqual("New element value", Element0[0], 42);
	Element0.Add(43);
	TestEqual("New element value", Element0[1], 43);
	TestEqual("New element array size", Element0.Num(), 2);

	TArrayAttribute<int> Element1 = Attributes.Get(1);
	Element1.Add(142);
	Element1.Add(143);
	Element1.Add(144);
	TestEqual("Second element array size", Element1.Num(), 3);

	Element0.Add(44);
	Element0.Add(44);
	Element0.Add(46);
	TestEqual("Check after insert 0", Element0[1], 43);
	TestEqual("Check after insert 1", Element0[2], 44);
	TestEqual("Check after insert 2", Element0[3], 44);
	TestEqual("Check after insert 3", Element0[4], 46);
	TestEqual("Check after insert 4", Element1[0], 142);

	Element0.InsertDefaulted(1, 3);
	TestEqual("Check after insert 5", Element0[0], 42);
	TestEqual("Check after insert 6", Element0[1], 0);
	TestEqual("Check after insert 7", Element0[4], 43);
	TestEqual("Check after insert 8", Element1[0], 142);
	TestEqual("Check after insert 9", Element0.Num(), 8);

	Element0.Reserve(20);
	TestEqual("Check after reserve", Element1[0], 142);

	AttributesSet.SetNumElements(255);
	TArrayAttribute<int> Element254 = Attributes.Get(254);
	Element254.Add(25400);

	TestEqual("Add element 254", Element254[0], 25400);
	TestEqual("Check element 200 empty", Attributes.Get(200).Num(), 0);

	AttributesSet.SetNumElements(256);
	TArrayAttribute<int> Element255 = Attributes.Get(255);
	Element255.Add(25500);
	TestEqual("Add element 255 1", Element254[0], 25400);
	TestEqual("Add element 255 2", Element255[0], 25500);

	AttributesSet.SetNumElements(300);
	TArrayAttribute<int> Element290 = Attributes.Get(290);
	Element290.Add(29000);
	TestEqual("Check element 256 empty", Attributes.Get(256).Num(), 0);
	TestEqual("Test element 290", Element290[0], 29000);

	AttributesSet.SetNumElements(522);
	TArrayAttribute<int> Element514 = Attributes.Get(514);
	Element514.Add(51400);
	TestEqual("Check element 512 empty", Attributes.Get(512).Num(), 0);
	TestEqual("Test element 514", Element514[0], 51400);

	Element0.Remove(44);
	TestEqual("Remove", Element0.Num(), 6);

	return true;
}
