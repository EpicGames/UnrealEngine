// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectToolsTests.h"

#include "Misc/Paths.h"
#include "Misc/AutomationTest.h"
#include "Misc/ScopeExit.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "ObjectTools.h"
#include "ScopedTransaction.h"
#include "Tests/AutomationTestSettings.h"
#include "Tests/AutomationEditorCommon.h"
#include "Tests/AutomationCommon.h"


class FObjectToolsTests_GatherObjectReferencersForDeletionTestBase : public FAutomationTestBase
{
public:
	FObjectToolsTests_GatherObjectReferencersForDeletionTestBase(const FString& InName, const bool bInComplexTask)
		: FAutomationTestBase(InName, bInComplexTask)
	{
	}

protected:
	void ResetUndoBuffer()
	{
		GEditor->Trans->Reset(FText::FromString("Discard undo history during ObjectTools testing."));
	}

	void TestGatherObjectReferencersForDeletion(UObject* Object, const TCHAR* MessageForReference, bool bExpectedIsReferencedValue, const TCHAR* MessageForTransactor, bool bExpectedIsReferencedByUndoValue)
	{
		bool bIsReferenced = false;
		bool bIsReferencedByUndo = false;
		ObjectTools::GatherObjectReferencersForDeletion(Object, bIsReferenced, bIsReferencedByUndo);
		TestEqual(MessageForReference, bIsReferenced, bExpectedIsReferencedValue);
		TestEqual(MessageForTransactor, bIsReferencedByUndo, bExpectedIsReferencedByUndoValue);
	}

	void TestGatherObjectReferencersForDeletion(UObject* Object, const TCHAR* Message, bool bExpectedIsReferencedValue, bool bExpectedIsReferencedByUndoValue)
	{
		TestGatherObjectReferencersForDeletion(Object, Message, bExpectedIsReferencedValue, Message, bExpectedIsReferencedByUndoValue);
	}

	enum class EReferenceType
	{
		Weak,
		Strong
	};

	enum class EReferenceChainType
	{
		Direct,
		Indirect
	};

	const TCHAR* LexToString(EReferenceType InReferenceType)
	{
		switch (InReferenceType)
		{
		case EReferenceType::Weak:
			return TEXT("Weak");
		case EReferenceType::Strong:
			return TEXT("Strong");
		default:
			checkNoEntry();
			return TEXT("Unknown");
		}
	}

	const TCHAR* LexToString(EReferenceChainType InReferenceChainType)
	{
		switch (InReferenceChainType)
		{
		case EReferenceChainType::Direct:
			return TEXT("Direct");
		case EReferenceChainType::Indirect:
			return TEXT("Indirect");
		default:
			checkNoEntry();
			return TEXT("Unknown");
		}
	}

	void SetObjectToolsTestObjectReference(UObjectToolsTestObject* InReferencer, UObject* InReferencee, EReferenceType InRerenceType)
	{
		switch (InRerenceType)
		{
			case EReferenceType::Weak:
				InReferencer->WeakReference = InReferencee;
				break;
			case EReferenceType::Strong:
				InReferencer->StrongReference = InReferencee;
				break;
			default:
				checkNoEntry();
				break;
		}
	}

	bool GetExpectedResultFromReferenceType(EReferenceType InRerenceType)
	{
		switch (InRerenceType)
		{
		case EReferenceType::Weak:
			return false;	// Weak reference are expected to not return a valid referencer
		case EReferenceType::Strong:
			return true;    // Strong reference are expected to return a valid referencer
		default:
			checkNoEntry();
			return false;
		}
	}

	bool GetExpectedTransactorResultFromReferenceType(EReferenceType InRerenceType)
	{
		switch (InRerenceType)
		{
		case EReferenceType::Weak:
			return true;	// The current behavior of the transactor is to convert weak reference to strong ones while they are in the undo buffer. This can be made false when the transactor behavior is fixed in the future.
		case EReferenceType::Strong:
			return true;    // Strong reference are expected to return a valid referencer
		default:
			checkNoEntry();
			return false;
		}
	}

	// Same boilerplate code that can be used with different types of references
	void TestWithTransactor(UPackage* TempPackage, UObjectToolsTestObject* InReferencer, UObject* InReferencee, EReferenceChainType InReferenceChainType, EReferenceType InReferenceType)
	{
		const bool bExpectedResult = GetExpectedResultFromReferenceType(InReferenceType);
		const TCHAR* ExpectedAction = bExpectedResult ? TEXT("detect") : TEXT("ignore");
		const bool bTransactorExpectedResult = GetExpectedTransactorResultFromReferenceType(InReferenceType);
		const TCHAR* ExpectedTransactorAction = bTransactorExpectedResult ? TEXT("detect") : TEXT("ignore");
		const FString InReferenceTypeName = FString(LexToString(InReferenceType)).ToLower();
		const FString InReferenceChainTypeName = FString(LexToString(InReferenceChainType)).ToLower();
		
		// Not strictly necessary, just a precaution in case some other test forgot to clear the undo buffer
		ResetUndoBuffer();

		// The property has not been set yet, we expect no referencer to be returned
		TestGatherObjectReferencersForDeletion(TempPackage, TEXT("GatherObjectReferencersForDeletion shouldn't detect any reference on any objects inside TempPackage"), false, false);

		// Set our reference
		SetObjectToolsTestObjectReference(InReferencer, InReferencee, InReferenceType);
		
		// Test the basic case with our reference now set
		TestGatherObjectReferencersForDeletion(TempPackage, *FString::Printf(TEXT("GatherObjectReferencersForDeletion should %s %s %s references on objects inside TempPackage"), ExpectedAction, *InReferenceChainTypeName, *InReferenceTypeName), bExpectedResult, false);

		// Set the reference to null in a transaction so that the transactor now holds onto it
		{
			FScopedTransaction Transaction(FText::FromString("ObjectTools testing."));
			InReferencer->PreEditChange(nullptr);
			SetObjectToolsTestObjectReference(InReferencer, nullptr, InReferenceType);
			InReferencer->PostEditChange();
		}

		// Validate that the transactor works as expected otherwise the test would be broken
		check(GEditor->Trans->IsObjectInTransactionBuffer(InReferencer));

		// The reference has been set to nullptr, only the transactor should now have a reference
		TestGatherObjectReferencersForDeletion(
			TempPackage, 
			*FString::Printf(TEXT("GatherObjectReferencersForDeletion shouldn't detect non-transactor %s %s references at this point"), *InReferenceChainTypeName, *InReferenceTypeName), false, 
			*FString::Printf(TEXT("GatherObjectReferencersForDeletion should %s %s %s references from the transactor"), ExpectedTransactorAction , *InReferenceChainTypeName, *InReferenceTypeName), bTransactorExpectedResult
		);

		// Restore the reference so that both the transactor and normal object have valid references
		SetObjectToolsTestObjectReference(InReferencer, InReferencee, InReferenceType);

		// Both the transactor and the object should now have a reference reported
		TestGatherObjectReferencersForDeletion(TempPackage, 
			*FString::Printf(TEXT("GatherObjectReferencersForDeletion should %s %s %s references on any objects inside TempPackage"), ExpectedAction, *InReferenceChainTypeName, *InReferenceTypeName), bExpectedResult, 
			*FString::Printf(TEXT("GatherObjectReferencersForDeletion should %s %s %s references from the transactor"), ExpectedTransactorAction, *InReferenceChainTypeName, *InReferenceTypeName), bTransactorExpectedResult);

		// This should clear the undo reference
		ResetUndoBuffer();

		// Test that the transactor is not considered a referencer anymore
		TestGatherObjectReferencersForDeletion(TempPackage, *FString::Printf(TEXT("GatherObjectReferencersForDeletion should %s %s %s references on objects inside TempPackage"), ExpectedAction, *InReferenceChainTypeName, *InReferenceTypeName), bExpectedResult, false);

		// Set the reference to null so that we exit the test in the same state we came in
		SetObjectToolsTestObjectReference(InReferencer, nullptr, InReferenceType);

		// Last check to make sure there is no more referencer left
		TestGatherObjectReferencersForDeletion(TempPackage, TEXT("GatherObjectReferencersForDeletion shouldn't detect any reference on any objects inside TempPackage"), false, false);
	}
};

/**
* Automation test for validating GatherObjectReferencersForDeletion functionality
*/
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(
	FObjectToolsTests_GatherObjectReferencersForDeletion, 
	FObjectToolsTests_GatherObjectReferencersForDeletionTestBase, 
	"Editor.ObjectTools.GatherObjectReferencersForDeletion", 
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FObjectToolsTests_GatherObjectReferencersForDeletion::RunTest(const FString& )
{
	UPackage* TempPackage = NewObject<UPackage>(nullptr, TEXT("/Engine/Editor/ObjectToolsTests/Transient"), RF_Transient);

	UObjectToolsTestObject* ObjectInPackage = NewObject<UObjectToolsTestObject>(TempPackage, NAME_None, RF_Transactional);

	//Make sure we do not use the legacy referencers for deletion
	static IConsoleVariable* CVarUseLegacyGetReferencersForDeletion = IConsoleManager::Get().FindConsoleVariable(TEXT("Editor.UseLegacyGetReferencersForDeletion"));
	check(CVarUseLegacyGetReferencersForDeletion);
	bool PreviousCVarUseLegacyGetReferencersForDeletionValue = CVarUseLegacyGetReferencersForDeletion->GetBool();
	CVarUseLegacyGetReferencersForDeletion->Set(0);
	//Put back the previous value when the test is done
	ON_SCOPE_EXIT
	{
		if (PreviousCVarUseLegacyGetReferencersForDeletionValue)
		{
			CVarUseLegacyGetReferencersForDeletion->Set(1);
		}
	};

	// Test direct references to ObjectInPackage
	{
		UObjectToolsTestObject* DirectRootReferencer = NewObject<UObjectToolsTestObject>(GetTransientPackage(), NAME_None, RF_Transactional);
		DirectRootReferencer->AddToRoot();

		TestWithTransactor(TempPackage, DirectRootReferencer, ObjectInPackage, EReferenceChainType::Direct, EReferenceType::Weak);
		TestWithTransactor(TempPackage, DirectRootReferencer, ObjectInPackage, EReferenceChainType::Direct, EReferenceType::Strong);

		DirectRootReferencer->RemoveFromRoot();
	}

	// Test indirect references to ObjectInPackage
	{
		UObjectToolsTestObject* Referencer = NewObject<UObjectToolsTestObject>(GetTransientPackage(), NAME_None, RF_Transactional);

		// This object itself is not rooted, so it's reference shouldn't matter
		Referencer->StrongReference = ObjectInPackage;

		UObjectToolsTestObject* IndirectRootReferencer = NewObject<UObjectToolsTestObject>(GetTransientPackage(), NAME_None, RF_Transactional);
		IndirectRootReferencer->AddToRoot();

		TestWithTransactor(TempPackage, IndirectRootReferencer, Referencer, EReferenceChainType::Indirect, EReferenceType::Weak);
		TestWithTransactor(TempPackage, IndirectRootReferencer, Referencer, EReferenceChainType::Indirect, EReferenceType::Strong);

		IndirectRootReferencer->RemoveFromRoot();
	}

	// Test when the transaction buffer has a direct reference to one of the objects we're trying to destroy
	{
		UObjectToolsTestObject* DummyObject = NewObject<UObjectToolsTestObject>(GetTransientPackage(), NAME_None, RF_Transactional);

		TestGatherObjectReferencersForDeletion(TempPackage, TEXT("GatherObjectReferencersForDeletion shouldn't detect any reference on any objects inside TempPackage"), false, false);

		{
			FScopedTransaction Transaction(FText::FromString("ObjectTools testing."));
			ObjectInPackage->PreEditChange(nullptr);
			ObjectInPackage->StrongReference = DummyObject;
			ObjectInPackage->PostEditChange();
		}

		TestGatherObjectReferencersForDeletion(
			TempPackage,
			TEXT("GatherObjectReferencersForDeletion shouldn't detect non-transactor references at this point"), false,
			TEXT("GatherObjectReferencersForDeletion should detect a transactor reference"), true
		);

		// Restore the property as it was before the test in case we add more test in the future
		ObjectInPackage->StrongReference = nullptr;

		// This should clear the undo reference
		ResetUndoBuffer();

		// Make sure we're back to initial state
		TestGatherObjectReferencersForDeletion(TempPackage, TEXT("GatherObjectReferencersForDeletion shouldn't detect any reference on any objects inside TempPackage"), false, false);
	}

	// Test object inside the package with RF_Standalone flag
	{
		UObjectToolsTestObject* StandaloneObject = NewObject<UObjectToolsTestObject>(TempPackage, NAME_None, RF_Transactional | RF_Standalone);

		TestGatherObjectReferencersForDeletion(TempPackage, TEXT("GatherObjectReferencersForDeletion should detect internal reference inside TempPackage"), true, false);

		StandaloneObject->ClearFlags(RF_Standalone);

		TestGatherObjectReferencersForDeletion(TempPackage, TEXT("GatherObjectReferencersForDeletion shouldn't detect any reference on any objects inside TempPackage"), false, false);
	}

	return true;
}

/**
* Automation test for validating ForceDeleteObjects functionality
*/
IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FObjectToolsTests_ForceDeleteObjects,
	"Editor.ObjectTools.ForceDeleteObjects",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter);

bool FObjectToolsTests_ForceDeleteObjects::RunTest(const FString&)
{
	// Simple test for now to repro an existing issue highlighting standalone usage on assets.
	// If we try to delete a group of objects having reference between each other with some of them
	// being stand-alone, the function doing the delete should ignore standalone referencers if 
	// the referencers are also part of the group of objects being deleted.

	UPackage* TempPackageA = NewObject<UPackage>(nullptr, TEXT("/Engine/Editor/ObjectToolsTests/Transient"), RF_Transient);
	UPackage* TempPackageB = NewObject<UPackage>(nullptr, TEXT("/Engine/Editor/ObjectToolsTests/Transient"), RF_Transient);

	UObjectToolsTestObject* ObjectInPackageA = NewObject<UObjectToolsTestObject>(TempPackageA, NAME_None, RF_Transactional | RF_Standalone);
	UObjectToolsTestObject* ObjectInPackageB = NewObject<UObjectToolsTestObject>(TempPackageB, NAME_None, RF_Transactional | RF_Standalone);

	ObjectInPackageA->StrongReference = ObjectInPackageB;
	ObjectInPackageB->StrongReference = ObjectInPackageA;

	TestEqual(
		TEXT("ForceDeleteObjects should be able to force delete both objects without warnings or errors"), 
		ObjectTools::ForceDeleteObjects({ ObjectInPackageA, ObjectInPackageB }),
		2
	);

	return true;
}
