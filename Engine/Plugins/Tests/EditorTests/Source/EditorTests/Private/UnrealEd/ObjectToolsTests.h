// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ObjectMacros.h"
#include "ObjectToolsTests.generated.h"

UCLASS()
class UObjectToolsTestObject : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TObjectPtr<UObject> StrongReference;

	UPROPERTY()
	TWeakObjectPtr<UObject> WeakReference;
};
