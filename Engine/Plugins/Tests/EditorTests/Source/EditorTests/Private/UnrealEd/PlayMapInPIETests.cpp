// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Editor.h"
#include "Engine/Texture.h"
#include "Misc/AutomationTest.h"
#include "Modules/ModuleManager.h"
#include "Engine/Engine.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ContentStreaming.h"
#include "EngineGlobals.h"
#include "Tests/AutomationCommon.h"
#include "Tests/AutomationEditorCommon.h"
#include "NavigationSystem.h"
#include "IAutomationControllerModule.h"

#if WITH_DEV_AUTOMATION_TESTS

#define LOCTEXT_NAMESPACE "PlayMapInPIE"

DEFINE_LOG_CATEGORY_STATIC(LogPlayMapInPIE, Log, All);

static UWorld* GetAnyGameWorld()
{
	UWorld* TestWorld = nullptr;
	const TIndirectArray<FWorldContext>& WorldContexts = GEngine->GetWorldContexts();
	for (const FWorldContext& Context : WorldContexts)
	{
		if (((Context.WorldType == EWorldType::PIE) || (Context.WorldType == EWorldType::Game)) && (Context.World() != NULL))
		{
			TestWorld = Context.World();
			break;
		}
	}

	return TestWorld;
}

DEFINE_LATENT_AUTOMATION_COMMAND(FLoadAllTextureLatentCommand);

bool FLoadAllTextureLatentCommand::Update()
{
	FlushAsyncLoading();

	// Make sure we finish all level streaming
	if (UWorld* GameWorld = GetAnyGameWorld())
	{
		GameWorld->FlushLevelStreaming(EFlushLevelStreamingType::Full);
	}

	// Force all mip maps to load.
	UTexture::ForceUpdateTextureStreaming();

	IStreamingManager::Get().StreamAllResources(0.0f);

	return true;
}

class FPlayMapInPIEBase : public FAutomationTestBase
{
public:
	FPlayMapInPIEBase(const FString& InName, const bool bInComplexTask)
		: FAutomationTestBase(InName, bInComplexTask)
	{
	}

	static void ParseTestMapInfo(const FString& Parameters, FString& MapObjectPath, FString& MapPackageName)
	{
		TArray<FString> ParamArray;
		Parameters.ParseIntoArray(ParamArray, TEXT(";"), true);

		MapObjectPath = ParamArray[0];
		MapPackageName = ParamArray[1];
	}

	/** 
	 * Requests a enumeration of all maps to be loaded
	 */
	virtual void GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const override
	{
		const UAutomationTestSettings* AutomationTestSettings = GetDefault<UAutomationTestSettings>();
		if (!AutomationTestSettings->bUseAllProjectMapsToPlayInPIE)
		{
			return;
		}

		IAssetRegistry& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

		if (!AssetRegistry.IsLoadingAssets())
		{
#if WITH_EDITOR
			static bool bDidScan = false;

			if (!GIsEditor && !bDidScan)
			{
				// For editor build -game, we need to do a full scan
				AssetRegistry.SearchAllAssets(true);
				bDidScan = true;
			}
#endif
			IAutomationControllerModule& AutomationControllerModule = FModuleManager::LoadModuleChecked<IAutomationControllerModule>(TEXT("AutomationController"));
			IAutomationControllerManagerPtr AutomationController = AutomationControllerModule.GetAutomationController();
			bool IsDeveloperDirectoryIncluded = AutomationController->IsDeveloperDirectoryIncluded();

			TArray<FAssetData> MapList;
			FARFilter Filter;
			Filter.ClassPaths.Add(UWorld::StaticClass()->GetClassPathName());
			Filter.bRecursiveClasses = true;
			Filter.bIncludeOnlyOnDiskAssets = true;
			if (AssetRegistry.GetAssets(Filter, /*out*/ MapList))
			{
				for (const FAssetData& MapAsset : MapList)
				{
					FString MapPackageName = MapAsset.PackageName.ToString();
					if (MapPackageName.Find(TEXT("/Game/")) == 0)
					{
						if (!IsDeveloperDirectoryIncluded && MapPackageName.Find(TEXT("/Game/Developers")) == 0) continue;

						FString MapAssetPath = MapAsset.GetObjectPathString();

						OutBeautifiedNames.Add(MapPackageName.RightChop(6).Replace(TEXT("/"), TEXT("."))); // Remove "/Game/" from the name and use dot syntax
						OutTestCommands.Add(MapAssetPath + TEXT(";") + MapPackageName);
					}
				}
			}
		}
	}

	virtual FString GetTestOpenCommand(const FString& Parameters) const override
	{
		FString MapObjectPath, MapPackageName;
		ParseTestMapInfo(Parameters, MapObjectPath, MapPackageName);

		return FString::Printf(TEXT("Automate.OpenMap %s"), *MapObjectPath);
	}

	virtual FString GetTestAssetPath(const FString& Parameters) const override
	{
		FString MapObjectPath, MapPackageName;
		ParseTestMapInfo(Parameters, MapObjectPath, MapPackageName);

		return MapObjectPath;
	}

	/**
	 * Execute the loading of each map and performance captures
	 *
	 * @param Parameters - Should specify which map name to load
	 * @return	TRUE if the test was successful, FALSE otherwise
	 */
	virtual bool RunTest(const FString& Parameters) override
	{
		FString MapObjectPath, MapPackageName;
		ParseTestMapInfo(Parameters, MapObjectPath, MapPackageName);

		bool bCanProceed = false;

		UWorld* TestWorld = GetAnyGameWorld();
		if (TestWorld && TestWorld->GetMapName() == MapPackageName)
		{
			// Map is already loaded.
			bCanProceed = true;
		}
		else
		{
			bCanProceed = AutomationOpenMap(MapPackageName);
		}

		if (bCanProceed)
		{
			ADD_LATENT_AUTOMATION_COMMAND(FLoadAllTextureLatentCommand());
			ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(0.25));
			ADD_LATENT_AUTOMATION_COMMAND(FWaitForShadersToFinishCompiling());
			ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(4.0));

			return true;
		}

		UE_LOGF(LogPlayMapInPIE, Error, "Failed to start the %ls map (possibly due to BP compilation issues)", *MapPackageName);
		return false;
	}

protected:
	virtual void SetTestContext(FString Context) override
	{
		FString MapObjectPath, MapPackageName;
		ParseTestMapInfo(Context, MapObjectPath, MapPackageName);
		TestParameterContext = MapPackageName.RightChop(6);
	}

};

// Editor only tests
IMPLEMENT_CUSTOM_COMPLEX_AUTOMATION_TEST(FPlayMapInPIE, FPlayMapInPIEBase, "Project.Maps.AllInPIE", (EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter))

void FPlayMapInPIE::GetTests(TArray<FString>& OutBeautifiedNames, TArray<FString>& OutTestCommands) const
{
	FPlayMapInPIEBase::GetTests(OutBeautifiedNames, OutTestCommands);
}

bool FPlayMapInPIE::RunTest(const FString& Parameters)
{
	return FPlayMapInPIEBase::RunTest(Parameters);
}

void OpenMap(const TArray<FString>& Args)
{
	if (Args.Num() != 1)
	{
		UE_LOGF(LogConsoleResponse, Display, "Automate.OpenMap failed, the number of arguments is wrong.  Automate.OpenMap MapObjectPath\n");
		return;
	}

	FString AssetPath(Args[0]);

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	FAssetData MapAssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(AssetPath));

	if (MapAssetData.IsValid())
	{
		bool bIsWorldAlreadyOpened = false;
		if (UWorld* EditorWorld = GEditor->GetEditorWorldContext().World())
		{
			if (FAssetData(EditorWorld).PackageName == MapAssetData.PackageName)
			{
				bIsWorldAlreadyOpened = true;
			}
		}

		if (!bIsWorldAlreadyOpened)
		{
			UObject* ObjectToEdit = MapAssetData.GetAsset();
			if (ObjectToEdit)
			{
				GEditor->EditObject(ObjectToEdit);
			}
		}
	}
}

FAutoConsoleCommand OpenMapAndFocusActorPIETestsCmd(
	TEXT("Automate.OpenMap"),
	TEXT("Opens a map."),
	FConsoleCommandWithArgsDelegate::CreateStatic(OpenMap)
);

#undef LOCTEXT_NAMESPACE

#endif //WITH_DEV_AUTOMATION_TESTS
