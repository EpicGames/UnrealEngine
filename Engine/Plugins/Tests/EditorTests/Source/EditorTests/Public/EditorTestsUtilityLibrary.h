// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Components/StaticMeshComponent.h"

#include "MeshMerge/MeshMergingSettings.h"
#include "MeshMerge/MeshProxySettings.h"

#include "EditorTestsUtilityLibrary.generated.h"

class UMaterialOptions;
class UWidget;
class UWidgetBlueprint;
struct FMeshMergingSettings;

/** Blueprint library for altering and analyzing animation / skeletal data */
UCLASS()
class UEditorTestsUtilityLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Bakes out material in-place for the given set of static mesh components using the MaterialMergeOptions */
	UFUNCTION(BlueprintCallable, Category = "MeshMergingLibrary|Test")
	static void BakeMaterialsForComponent(UStaticMeshComponent* InStaticMeshComponent, const UMaterialOptions* MaterialOptions, const UMaterialMergeOptions* MaterialMergeOptions);

	/** Merges meshes and bakes out materials into a atlas-material for the given set of static mesh components using the MergeSettings */
	UFUNCTION(BlueprintCallable, Category = "MeshMergingLibrary|Test")
	static void MergeStaticMeshComponents(const TArray<UStaticMeshComponent*>& InStaticMeshComponents, const FMeshMergingSettings& MergeSettings, const bool bReplaceActors, TArray<int32>& OutLODIndices);

	/** Simplify meshes and bakes out materials into a atlas-material for the given set of static mesh components using the ProxySettings */
	UFUNCTION(BlueprintCallable, Category = "MeshMergingLibrary|Test")
	static void CreateProxyMesh(const TArray<UStaticMeshComponent*>& InStaticMeshComponents, const struct FMeshProxySettings& ProxySettings);

	/** Finds a UWidget object used by the editor - useful for testing widget editing */
	UFUNCTION(BlueprintCallable, Category = "User Interface|Test")
	static UWidget* GetChildEditorWidgetByName(UWidgetBlueprint* WidgetBlueprint, FString Name);

	/** Simple function for setting UWidget::Navigation, which has a details customization */
	UFUNCTION(BlueprintCallable, Category = "User Interface|Test")
	static void SetEditorWidgetNavigationRule(UWidget* Widget, EUINavigation Nav, EUINavigationRule Rule);

	/** Simple logic for getting data within UWidget::Navigation, which has a details customization */
	UFUNCTION(BlueprintCallable, Category = "User Interface|Test")
	static EUINavigationRule GetEditorWidgetNavigationRule(UWidget* Widget, EUINavigation Nav);
};
